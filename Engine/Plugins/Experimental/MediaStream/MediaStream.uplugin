{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Media Stream",
	"Description": "Content/type agnostic chainable media proxy with media player integration.",
	"Category": "Experimental",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "MediaStream",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MediaStreamEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [

		{
			"Name": "LevelSequenceEditor",
			"Enabled": true
		},
		{
			"Name": "MediaCompositing",
			"Enabled": true
		},
		{
			"Name": "MediaPlayerEditor",
			"Enabled": true
		}
	]
}
