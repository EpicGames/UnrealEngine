// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MediaStream : ModuleRules
{
	public MediaStream(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Engine",
				"MediaAssets"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"LevelSequence",
				"MediaCompositing",
				"MediaUtils",
				"MovieScene"
			}
		);

		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"DesktopWidgets",
					"PropertyEditor",
					"Slate",
					"SlateCore",
					"UnrealEd"
				}
			);
		}
	}
}
