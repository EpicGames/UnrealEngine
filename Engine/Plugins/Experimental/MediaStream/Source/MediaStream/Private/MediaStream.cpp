// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaStream.h"

#include "MediaPlaylist.h"
#include "MediaSource.h"
#include "MediaSourceOptions.h"
#include "MediaStreamModule.h"
#include "MediaStreamSourceBlueprintLibrary.h"
#include "MovieScene.h"
#include "Players/MediaStreamLocalPlayer.h"
#include "Players/MediaStreamProxyPlayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaStream)

FName UMediaStream::GetSourcePropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UMediaStream, Source);
}

FName UMediaStream::GetPlayerPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UMediaStream, PlayerObject);
}

bool UMediaStream::HasValidSource() const
{
	return UMediaStreamSourceBlueprintLibrary::IsValidMediaSource(Source);
}

const FMediaStreamSource& UMediaStream::GetSource() const
{
	return Source;
}

const FMediaStreamSource& UMediaStream::ResolveSource() const
{
	if (UMediaStreamProxyPlayer* ProxyPlayer = Cast<UMediaStreamProxyPlayer>(PlayerObject))
	{
		if (UMediaStream* ProxyStream = ProxyPlayer->GetSourceStream())
		{
			return ProxyStream->ResolveSource();
		}
	}

	return Source;
}

bool UMediaStream::SetSource(const FMediaStreamSource& InSource)
{
	if (bIsApplyingSource)
	{
		// The outer ApplySource owns the Source field and player lifecycle; nested mutation would leave them inconsistent.
		UE_LOGF(LogMediaStream, Warning,
			"UMediaStream::SetSource ignored: called recursively from a delegate handler running inside another SetSource. "
			"Stream=\"%ls\", AttemptedSource={Scheme=%ls, Path=\"%ls\", Object=\"%ls\"}.",
			*GetPathName(),
			*InSource.Scheme.ToString(),
			*InSource.Path,
			*GetNameSafe(InSource.Object));
		return false;
	}

	if (Source == InSource)
	{
		return true;
	}

	Source = InSource;

	return ApplySource();
}

TScriptInterface<IMediaStreamPlayer> UMediaStream::GetPlayer() const
{
	return PlayerObject;
}

bool UMediaStream::EnsurePlayer(bool bInForceRecreatePlayer)
{
	if (bInForceRecreatePlayer)
	{
		FMediaStreamSource SourceTemp = Source;
		Source = FMediaStreamSource();
		
		if (IsValid(PlayerObject))
		{
			if (IMediaStreamPlayer* Player = Cast<IMediaStreamPlayer>(PlayerObject))
			{
				Player->Deinitialize();
				Player->OnSourceChanged(Source);
				PlayerObject = nullptr;
			}
		}

		SetSource(SourceTemp);
	}
	else if (!PlayerObject && UMediaStreamSourceBlueprintLibrary::IsValidMediaSource(Source))
	{
		ApplySource();
	}

	return !!PlayerObject;
}

UMediaStream::FOnSourceChanged& UMediaStream::GetOnSourceChanged()
{
	return OnSourceChanged;
}

UMediaStream::FOnPlayerChanged& UMediaStream::GetOnPlayerChanged()
{
	return OnPlayerChanged;
}

UMediaStream::FOnPreSourceChange& UMediaStream::GetOnPreSourceChange()
{
	return OnPreSourceChange;
}

void UMediaStream::Close()
{
	SetSource(FMediaStreamSource());
}

#if WITH_EDITOR
void UMediaStream::PreEditUndo()
{
	Super::PreEditUndo();

	Source_PreUndo = Source;
}

void UMediaStream::PostEditUndo()
{
	Super::PostEditUndo();

	if (Source_PreUndo != Source)
	{
		ApplySource();
	}
}

void UMediaStream::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMediaStream, Source))
	{
		ApplySource();
	}
}
#endif

void UMediaStream::PostDuplicate(bool bInDuplicateForPIE)
{
	Super::PostDuplicate(bInDuplicateForPIE);

	ApplySource();
}

void UMediaStream::PostEditImport()
{
	Super::PostEditImport();

	ApplySource();
}

void UMediaStream::PostNetReceive()
{
	Super::PostNetReceive();

	ApplySource();
}

void UMediaStream::BeginDestroy()
{
	Super::BeginDestroy();

	Source = FMediaStreamSource();

	if (IsValid(PlayerObject))
	{
		if (IMediaStreamPlayer* Player = Cast<IMediaStreamPlayer>(PlayerObject))
		{
			Player->Deinitialize();
		}
	}
}

float UMediaStream::GetProxyRate() const
{
	if (IMediaStreamPlayer* MediaStreamPlayer = GetPlayer().GetInterface())
	{
		if (UMediaPlayer* MediaPlayer = MediaStreamPlayer->GetPlayer())
		{
			return MediaPlayer->GetRate();
		}
	}

	return 0.f;
}

bool UMediaStream::SetProxyRate(float Rate)
{
	if (IMediaStreamPlayer* MediaStreamPlayer = GetPlayer().GetInterface())
	{
		if (MediaStreamPlayer->IsReadOnly())
		{
			return false;
		}

		if (UMediaPlayer* MediaPlayer = MediaStreamPlayer->GetPlayer())
		{
			return MediaPlayer->SetRate(Rate);
		}
	}

	return false;
}

bool UMediaStream::IsExternalControlAllowed()
{
	if (IMediaStreamPlayer* MediaStreamPlayer = GetPlayer().GetInterface())
	{
		return !MediaStreamPlayer->IsReadOnly();
	}

	return false;
}

const FMediaSourceCacheSettings& UMediaStream::GetCacheSettings() const
{
	static FMediaSourceCacheSettings DefaultCacheSetting = {/* Override */ false, /* Cache ahead */ 0.f};
	return DefaultCacheSetting;
}

UMediaSource* UMediaStream::ProxyGetMediaSourceFromIndex(int32 Index) const
{
	const FMediaStreamSource& ResolvedSource = ResolveSource();

	if (UMediaPlaylist* Playlist = Cast<UMediaPlaylist>(ResolvedSource.Object))
	{
		return Playlist->Get(Index);
	}

	if (Index != 0)
	{
		return nullptr;
	}

	return Cast<UMediaSource>(ResolvedSource.Object);
}

UMediaTexture* UMediaStream::ProxyGetMediaTexture(int32 LayerIndex, int32 TextureIndex)
{
	if (IMediaStreamPlayer* MediaStreamPlayer = GetPlayer().GetInterface())
	{
		return MediaStreamPlayer->GetMediaTexture();
	}

	return nullptr;
}

void UMediaStream::ProxyReleaseMediaTexture(int32 LayerIndex, int32 TextureIndex)
{
	// Do nothing, we handle these internally.
}

bool UMediaStream::ProxySetAspectRatio(UMediaPlayer* InMediaPlayer)
{
	// We have no aspect ratio settings
	return false;
}

void UMediaStream::ProxySetTextureBlend(int32 LayerIndex, int32 TextureIndex, float Blend)
{
	// Do nothing, we have no blending
}

bool UMediaStream::ApplySource()
{
	TGuardValue<bool> ApplyingGuard(bIsApplyingSource, true);

	if (!UMediaStreamSourceBlueprintLibrary::IsValidMediaSource(Source))
	{
		if (IsValid(PlayerObject))
		{
			if (IMediaStreamPlayer* Player = Cast<IMediaStreamPlayer>(PlayerObject))
			{
				Player->Deinitialize();
				Player->OnSourceChanged(Source);
			}
		}

		OnSourceChanged.Broadcast(this);

		return false;
	}

	const bool bIsProxyStream = IsValid(Source.Object) && (Source.Object->GetClass() == UMediaStream::StaticClass());

	if (!IsValid(PlayerObject))
	{
		if (bIsProxyStream)
		{
			PlayerObject = NewObject<UMediaStreamProxyPlayer>(this);
		}
		else
		{
			PlayerObject = NewObject<UMediaStreamLocalPlayer>(this);
		}

		Cast<IMediaStreamPlayer>(PlayerObject)->OnCreated();
	}
	else
	{
		if (bIsProxyStream && !PlayerObject->IsA<UMediaStreamProxyPlayer>())
		{
			if (UMediaStreamLocalPlayer* LocalPlayer = Cast<UMediaStreamLocalPlayer>(PlayerObject))
			{
				LocalPlayer->Deinitialize();
			}

			PlayerObject = NewObject<UMediaStreamProxyPlayer>(this);
			Cast<IMediaStreamPlayer>(PlayerObject)->OnCreated();
		}
		else if (!bIsProxyStream && !PlayerObject->IsA<UMediaStreamLocalPlayer>())
		{
			if (UMediaStreamProxyPlayer* ProxyPlayer = Cast<UMediaStreamProxyPlayer>(PlayerObject))
			{
				ProxyPlayer->Deinitialize();
			}

			PlayerObject = NewObject<UMediaStreamLocalPlayer>(this);
			Cast<IMediaStreamPlayer>(PlayerObject)->OnCreated();
		}
	}

	if (IMediaStreamPlayer* Player = Cast<IMediaStreamPlayer>(PlayerObject))
	{
		OnPreSourceChange.Broadcast(this);

		Player->OnSourceChanged(Source);
	}

	OnSourceChanged.Broadcast(this);

	return true;
}
