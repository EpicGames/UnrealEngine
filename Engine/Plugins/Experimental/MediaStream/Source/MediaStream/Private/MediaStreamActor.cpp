// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaStreamActor.h"

#include "MediaStreamComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaStreamActor)

AMediaStreamActor::AMediaStreamActor()
{
	MediaStreamComponent = CreateDefaultSubobject<UMediaStreamComponent>(TEXT("MediaStreamComponent"));
	SetRootComponent(MediaStreamComponent);
}
