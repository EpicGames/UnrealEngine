// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaStreamComponent.h"

#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "IMediaStreamPlayer.h"
#include "MediaStream.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaStreamComponent)

namespace UE::MediaStream::Private
{
	const FLazyName TextureParameterName = TEXT("MediaTexture");
	constexpr const TCHAR* DefaultTexturePath = TEXT("/Script/Engine.Texture2D'/Engine/EditorMaterials/MatineeGroups/MAT_Groups_Anim.MAT_Groups_Anim'");
	constexpr const TCHAR* BaseMaterialPath = TEXT("/Script/Engine.Material'/MediaStream/MediaStreamComponentMaterial.MediaStreamComponentMaterial'");
	constexpr const TCHAR* StaticMeshPath = TEXT("/Script/Engine.StaticMesh'/Engine/EditorMeshes/EditorPlane.EditorPlane'");

	UTexture* GetDefaultTexture()
	{
		const TSoftObjectPtr<UTexture> StaticMeshPtr = TSoftObjectPtr<UTexture>(FSoftObjectPath(DefaultTexturePath));
		return StaticMeshPtr.LoadSynchronous();
	}

	UMaterial* GetBaseMaterial()
	{
		const TSoftObjectPtr<UMaterial> StaticMeshPtr = TSoftObjectPtr<UMaterial>(FSoftObjectPath(BaseMaterialPath));
		return StaticMeshPtr.LoadSynchronous();
	}

	UStaticMesh* GetStaticMesh()
	{
		const TSoftObjectPtr<UStaticMesh> StaticMeshPtr = TSoftObjectPtr<UStaticMesh>(FSoftObjectPath(StaticMeshPath));
		return StaticMeshPtr.LoadSynchronous();
	}
}

UMediaStreamComponent::UMediaStreamComponent()
{
	using namespace UE::MediaStream::Private;

	MediaStream = CreateDefaultSubobject<UMediaStream>("MediaStream");
}

void UMediaStreamComponent::PostInitProperties()
{
 	Super::PostInitProperties();

	SetStaticMesh(UE::MediaStream::Private::GetStaticMesh());

	MediaStream->GetOnSourceChanged().AddDynamic(this, &UMediaStreamComponent::OnSourceChanged);

	InitPlayer();
}

void UMediaStreamComponent::PostLoad()
{
	Super::PostLoad();

	InitPlayer();
}

void UMediaStreamComponent::PostNetReceive()
{
	Super::PostNetReceive();

	InitPlayer();
}

void UMediaStreamComponent::OnSourceChanged(UMediaStream* InMediaStream)
{
	InitPlayer();
}

void UMediaStreamComponent::InitPlayer()
{
	using namespace UE::MediaStream::Private;

	UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(MaterialInstance);

	if (!MID)
	{
		MaterialInstance = UMaterialInstanceDynamic::Create(GetBaseMaterial(), GetTransientPackage());
		MID = Cast<UMaterialInstanceDynamic>(MaterialInstance);
	}

	if (GetMaterial(0) != MaterialInstance)
	{
		SetMaterial(0, MaterialInstance);
		MarkRenderStateDirty();
	}

	IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface();

	if (!MediaStreamPlayer)
	{
		MID->SetTextureParameterValue(TextureParameterName, GetDefaultTexture());
		return;
	}

	UMediaTexture* MediaTexture = MediaStreamPlayer->GetMediaTexture();

	if (!MediaTexture)
	{
		MID->SetTextureParameterValue(TextureParameterName, GetDefaultTexture());
		return;
	}

	MID->SetTextureParameterValue(TextureParameterName, MediaTexture);
}
