// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaStreamModule.h"

#include "HAL/IConsoleManager.h"
#include "MediaStreamObjectHandlerManager.h"
#include "MediaStreamSchemeHandlerManager.h"
#include "Modules/ModuleManager.h"
#include "ObjectHandlers/MediaStreamMediaPlaylistHandler.h"
#include "ObjectHandlers/MediaStreamMediaSourceHandler.h"
#include "ObjectHandlers/MediaStreamMediaStreamHandler.h"
#include "SchemeHandlers/MediaStreamAssetSchemeHandler.h"
#include "SchemeHandlers/MediaStreamFileSchemeHandler.h"
#include "SchemeHandlers/MediaStreamManagedSchemeHandler.h"
#include "SchemeHandlers/MediaStreamSubobjectSchemeHandler.h"
#include "UObject/UObjectBase.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

DEFINE_LOG_CATEGORY(LogMediaStream);

#if WITH_EDITOR
namespace UE::MediaStream
{
	TAutoConsoleVariable<bool> CVarAutoLoadMediaStreamsOnMapLoad = TAutoConsoleVariable<bool>(
		TEXT("MediaStream.OpenOnMapLoad"),
		/* Default */ true,
		TEXT("Automatically loads media streams when loading a level. If set to false, media streams must be manually loaded. Only applies to the Level Editor, not PIE or runtime.")
	);
}
#endif

FMediaStreamModule& FMediaStreamModule::Get()
{
	return FModuleManager::Get().GetModuleChecked<FMediaStreamModule>(UE_MODULE_NAME);
}

bool FMediaStreamModule::CanOpenSourceOnLoad()
{
#if WITH_EDITOR
	using namespace UE::MediaStream;

	return !GIsEditor || !bIsMapLoading || CVarAutoLoadMediaStreamsOnMapLoad.GetValueOnAnyThread();
#else
	return true;
#endif
}

bool FMediaStreamModule::CanAutoplay() const
{
#if WITH_EDITOR
	using namespace UE::MediaStream;

	return !GIsEditor || !bIsMapLoading || CVarAutoLoadMediaStreamsOnMapLoad.GetValueOnAnyThread();
#else
	return true;
#endif	
}

void FMediaStreamModule::StartupModule()
{
	FMediaStreamSchemeHandlerManager& SchemeHandlerManager = FMediaStreamSchemeHandlerManager::Get();
	SchemeHandlerManager.RegisterSchemeHandler<FMediaStreamFileSchemeHandler>();
	SchemeHandlerManager.RegisterSchemeHandler<FMediaStreamAssetSchemeHandler>();
	SchemeHandlerManager.RegisterSchemeHandler<FMediaStreamManagedSchemeHandler>();
	SchemeHandlerManager.RegisterSchemeHandler<FMediaStreamSubobjectSchemeHandler>();

	FMediaStreamObjectHandlerManager& ObjectHandlerManager = FMediaStreamObjectHandlerManager::Get();
	ObjectHandlerManager.RegisterObjectHandler<FMediaStreamMediaPlaylistHandler>();
	ObjectHandlerManager.RegisterObjectHandler<FMediaStreamMediaSourceHandler>();
	ObjectHandlerManager.RegisterObjectHandler<FMediaStreamMediaStreamHandler>();

#if WITH_EDITOR
	bIsMapLoading = false;

	FEditorDelegates::OnMapLoad.AddRaw(this, &FMediaStreamModule::OnMapLoad);
	FEditorDelegates::OnMapOpened.AddRaw(this, &FMediaStreamModule::OnMapOpened);
#endif
}

void FMediaStreamModule::ShutdownModule()
{
	if (!UObjectInitialized())
	{
		return;
	}

	FMediaStreamSchemeHandlerManager& SchemeHandlerManager = FMediaStreamSchemeHandlerManager::Get();
	SchemeHandlerManager.UnregisterSchemeHandler<FMediaStreamAssetSchemeHandler>();
	SchemeHandlerManager.UnregisterSchemeHandler<FMediaStreamFileSchemeHandler>();
	SchemeHandlerManager.UnregisterSchemeHandler<FMediaStreamManagedSchemeHandler>();
	SchemeHandlerManager.UnregisterSchemeHandler<FMediaStreamSubobjectSchemeHandler>();

	FMediaStreamObjectHandlerManager& ObjectHandlerManager = FMediaStreamObjectHandlerManager::Get();
	ObjectHandlerManager.UnregisterObjectHandler<FMediaStreamMediaPlaylistHandler>();
	ObjectHandlerManager.UnregisterObjectHandler<FMediaStreamMediaSourceHandler>();
	ObjectHandlerManager.UnregisterObjectHandler<FMediaStreamMediaStreamHandler>();

#if WITH_EDITOR
	FEditorDelegates::OnMapLoad.RemoveAll(this);
	FEditorDelegates::OnMapOpened.RemoveAll(this);
#endif
}

#if WITH_EDITOR
void FMediaStreamModule::OnMapLoad(const FString& InFilename, FCanLoadMap& OutCanLoadMap)
{
	// If this has been set to false, don't set the flag.
	if (OutCanLoadMap.Get())
	{
		bIsMapLoading = true;
	}
}

void FMediaStreamModule::OnMapOpened(const FString& InFilename, bool bInTemplate)
{
	bIsMapLoading = false;
}
#endif

IMPLEMENT_MODULE(FMediaStreamModule, MediaStream);
