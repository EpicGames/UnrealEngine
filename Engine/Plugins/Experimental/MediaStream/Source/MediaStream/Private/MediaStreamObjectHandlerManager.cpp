// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaStreamObjectHandlerManager.h"

#include "Engine/Engine.h"
#include "IMediaAssetsModule.h"
#include "IMediaStreamObjectHandler.h"
#include "MediaPlayer.h"
#include "MediaPlaylist.h"
#include "MediaSource.h"
#include "MediaStream.h"
#include "MediaStreamModule.h"
#include "MediaStreamUtils.h"
#include "Modules/ModuleManager.h"

FMediaStreamObjectHandlerManager& FMediaStreamObjectHandlerManager::Get()
{
	static FMediaStreamObjectHandlerManager Manager;
	return Manager;
}

bool FMediaStreamObjectHandlerManager::CanHandleObject(const UClass* InClass) const
{
	if (!IsValid(InClass))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Class in FMediaStreamObjectHandlerManager::CanHandleObject");
		return false;
	}

	return GetHandler(InClass).IsValid();
}

bool FMediaStreamObjectHandlerManager::CanHandleObject(const UObject* InObject) const
{
	if (!IsValid(InObject))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Object in FMediaStreamObjectHandlerManager::CanHandleObject");
		return false;
	}

	return CanHandleObject(InObject->GetClass());
}

UMediaPlayer* FMediaStreamObjectHandlerManager::CreateOrUpdatePlayer(const FMediaStreamObjectHandlerCreatePlayerParams& InParams) const
{
	if (!IsValid(InParams.MediaStream))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream in FMediaStreamObjectHandlerManager::CreateOrUpdatePlayer");
		return nullptr;
	}

	if (!IsValid(InParams.Source))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Object in FMediaStreamObjectHandlerManager::CreateOrUpdatePlayer");
		return nullptr;
	}

	if (InParams.CurrentPlayer)
	{
		if (UMediaSource* MediaSource = Cast<UMediaSource>(InParams.Source))
		{
			const bool bIsValidPlayer = InParams.bCanOpenSource
				? InParams.CurrentPlayer->OpenSourceWithOptions(MediaSource, UE::MediaStream::Utils::MakePlayerOptions(InParams.MediaStream))
				: InParams.CurrentPlayer->CanPlaySource(MediaSource);

			if (bIsValidPlayer)
			{
				return InParams.CurrentPlayer;
			}
		}
		else if (UMediaPlaylist* Playlist = Cast<UMediaPlaylist>(InParams.Source))
		{
			UMediaSource* FirstPlaylistEntry = Playlist->Get(0);

			if (InParams.bCanOpenSource)
			{
				// For OpenPlaylist, there is no variant of the function where we can give the player options, 
				// so we bind the texture to the player instead.
				UE::MediaStream::Utils::BindMediaTextureToPlayer(InParams.MediaStream, InParams.CurrentPlayer);
			}

			const bool bIsValidPlayer = InParams.bCanOpenSource
				? InParams.CurrentPlayer->OpenPlaylist(Playlist)
				: (!FirstPlaylistEntry || InParams.CurrentPlayer->CanPlaySource(FirstPlaylistEntry));

			if (bIsValidPlayer)
			{
				return InParams.CurrentPlayer;
			}
		}
	}

	if (TSharedPtr<IMediaStreamObjectHandler> Handler = GetHandler(InParams.Source->GetClass()))
	{
		UMediaPlayer* MediaPlayer = Handler->CreateOrUpdatePlayer(InParams);

		if (!MediaPlayer)
		{
			UE_LOGF(LogMediaStream, Error, "Failed to create Media Player in FMediaStreamObjectHandlerManager::CreateOrUpdatePlayer [%ls] [%ls]", *InParams.Source->GetClass()->GetName(), *InParams.Source->GetName());
			return nullptr;
		}

		return MediaPlayer;
	}

	// We cannot allow players to open sources, which we cannot control beyond this point.
	if (!InParams.bCanOpenSource)
	{
		UE_LOGF(LogMediaStream, Error, "Cannot use Media Assets Module to create player at the moment in FMediaStreamObjectHandlerManager::CreateOrUpdatePlayer [%ls] [%ls]", *InParams.Source->GetClass()->GetName(), *InParams.Source->GetName());
		return nullptr;
	}

	if (IMediaAssetsModule* MediaAssetsModule = FModuleManager::LoadModulePtr<IMediaAssetsModule>("MediaAssets"))
	{
		UObject* PlayerProxy = nullptr;
		UMediaPlayer* MediaPlayer = MediaAssetsModule->GetPlayerFromObject(InParams.Source, PlayerProxy);

		if (!MediaPlayer)
		{
			UE_LOGF(LogMediaStream, Error, "No player for Object in FMediaStreamObjectHandlerManager::CreateOrUpdatePlayer [%ls] [%ls]", *InParams.Source->GetClass()->GetName(), *InParams.Source->GetName());
			return nullptr;
		}

		return MediaPlayer;
	}

	UE_LOGF(LogMediaStream, Error, "Unable to get Media Assets Module in FMediaStreamObjectHandlerManager::CreateOrUpdatePlayer [%ls] [%ls]", *InParams.Source->GetClass()->GetName(), *InParams.Source->GetName());
	return nullptr;
}

bool FMediaStreamObjectHandlerManager::HasObjectHandler(const UClass* InClass) const
{
	if (!IsValid(InClass))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Class in FMediaStreamObjectHandlerManager::HasObjectHandler");
		return false;
	}

	return Handlers.Contains(InClass->GetFName());
}

bool FMediaStreamObjectHandlerManager::HasObjectHandler(const UObject* InObject) const
{
	if (!IsValid(InObject))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Object in FMediaStreamObjectHandlerManager::HasObjectHandler");
		return false;
	}

	return HasObjectHandler(InObject->GetClass());
}

TSharedPtr<IMediaStreamObjectHandler> FMediaStreamObjectHandlerManager::GetHandler(const UClass* InClass) const
{
	for (const UClass* Class = InClass; Class; Class = Class->GetSuperClass())
	{
		const FName ClassName = Class->GetFName();

		if (const TSharedRef<IMediaStreamObjectHandler>* HandlerPtr = Handlers.Find(ClassName))
		{
			return *HandlerPtr;
		}
	}

	return nullptr;
}

bool FMediaStreamObjectHandlerManager::RegisterObjectHandler(const UClass* InClass, const TSharedRef<IMediaStreamObjectHandler>& InHandler)
{
	if (!IsValid(InClass))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Class in FMediaStreamObjectHandlerManager::RegisterObjectHandler");
		return false;
	}

	const FName ClassName = InClass->GetFName();

	if (Handlers.Contains(ClassName))
	{
		return false;
	}

	Handlers.Add(ClassName, InHandler);

	return true;
}

TSharedPtr<IMediaStreamObjectHandler> FMediaStreamObjectHandlerManager::UnregisterObjectHandler(const UClass* InClass)
{
	if (!IsValid(InClass))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Class in FMediaStreamObjectHandlerManager::UnregisterObjectHandler");
		return nullptr;
	}

	const FName ClassName = InClass->GetFName();

	if (const TSharedRef<IMediaStreamObjectHandler>* HandlerPtr = Handlers.Find(ClassName))
	{
		TSharedRef<IMediaStreamObjectHandler> Handler = *HandlerPtr;
		Handlers.Remove(ClassName);

		return Handler;
	}

	return nullptr;
}
