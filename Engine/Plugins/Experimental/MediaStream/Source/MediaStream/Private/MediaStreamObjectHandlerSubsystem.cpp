// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaStreamObjectHandlerSubsystem.h"

#include "Engine/Engine.h"
#include "MediaStreamModule.h"
#include "MediaStreamObjectHandlerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaStreamObjectHandlerSubsystem)

UMediaStreamObjectHandlerSubsystem* UMediaStreamObjectHandlerSubsystem::Get()
{
	if (!UObjectInitialized())
	{
		UE_LOGF(LogMediaStream, Error, "UObject system not initialized in UMediaStreamObjectHandlerSubsystem::Get");
		return nullptr;
	}

	if (!GEngine)
	{
		UE_LOGF(LogMediaStream, Error, "Invalid GEngine in UMediaStreamObjectHandlerSubsystem::Get");
		return nullptr;
	}

	UMediaStreamObjectHandlerSubsystem* ObjectHandlerSubsystem = GEngine->GetEngineSubsystem<UMediaStreamObjectHandlerSubsystem>();

	if (!ObjectHandlerSubsystem)
	{
		UE_LOGF(LogMediaStream, Error, "Unable to find Media Source Object Handler Subsystem in UMediaStreamObjectHandlerSubsystem::Get");
		return nullptr;
	}

	return ObjectHandlerSubsystem;
}

bool UMediaStreamObjectHandlerSubsystem::CanHandleObject(const UClass* InClass) const
{
	return FMediaStreamObjectHandlerManager::Get().CanHandleObject(InClass);
}

UMediaPlayer* UMediaStreamObjectHandlerSubsystem::CreateMediaPlayer(const FMediaStreamObjectHandlerCreatePlayerParams& InParams) const
{
	return FMediaStreamObjectHandlerManager::Get().CreateOrUpdatePlayer(InParams);
}

bool UMediaStreamObjectHandlerSubsystem::HasObjectHandler(const UClass* InClass) const
{
	return FMediaStreamObjectHandlerManager::Get().HasObjectHandler(InClass);
}
