// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaStreamPlayerConfig.h"

#include "MediaPlayer.h"
#include "MediaPlayerFacade.h"
#include "MediaPlayerOptions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaStreamPlayerConfig)

bool FMediaStreamPlayerConfig::operator==(const FMediaStreamPlayerConfig& InOther) const
{
	return bPlayOnOpen == InOther.bPlayOnOpen
		&& TrackOptions.Video == InOther.TrackOptions.Video
		&& TrackOptions.Audio == InOther.TrackOptions.Audio
		&& TrackOptions.Caption == InOther.TrackOptions.Caption
		&& TrackOptions.Metadata == InOther.TrackOptions.Metadata
		&& TrackOptions.Script == InOther.TrackOptions.Script
		&& TrackOptions.Subtitle == InOther.TrackOptions.Subtitle
		&& TrackOptions.Text == InOther.TrackOptions.Text
		&& bLooping == InOther.bLooping
		&& Volume == InOther.Volume
		&& PlaybackTimeRange == InOther.PlaybackTimeRange
		&& Rate == InOther.Rate
		&& TimeDelay == InOther.TimeDelay
		&& bShuffle == InOther.bShuffle
		&& CacheAhead == InOther.CacheAhead
		&& CacheBehind == InOther.CacheBehind
		&& CacheBehindGame == InOther.CacheBehindGame
		&& bUseLocalCache == InOther.bUseLocalCache;
}

FMediaPlayerOptions FMediaStreamPlayerConfig::CreateOptions(const FTimespan& InStartTime, const TMap<FName, FVariant>& InCustomOptions) const
{
	FMediaPlayerOptions Options;
	Options.Tracks = TrackOptions;
	Options.TrackSelection = EMediaPlayerOptionTrackSelectMode::UseTrackOptionIndices;
	Options.SeekTime = InStartTime;
	Options.SeekTimeType = EMediaPlayerOptionSeekTimeType::RelativeToStartTime;
	Options.PlayOnOpen = bPlayOnOpen ? EMediaPlayerOptionBooleanOverride::Enabled : EMediaPlayerOptionBooleanOverride::Disabled;
	Options.Loop = bLooping ? EMediaPlayerOptionBooleanOverride::Enabled : EMediaPlayerOptionBooleanOverride::Disabled;

	// ImgMediaPlayer reads its smart-cache settings from InternalCustomOptions; reuse CacheAhead
	// as the look-ahead window so a single config field drives both the generic UMediaPlayer
	// cache and the ImgMedia per-player cache.
	Options.InternalCustomOptions.Emplace(MediaPlayerOptionValues::ImgMediaSmartCacheEnabled(), FVariant(bUseLocalCache));
	Options.InternalCustomOptions.Emplace(MediaPlayerOptionValues::ImgMediaSmartCacheTimeToLookAhead(), FVariant(static_cast<double>(CacheAhead)));

	// Caller-provided extras win on key collision.
	Options.InternalCustomOptions.Append(InCustomOptions);

	return Options;
}

void FMediaStreamPlayerConfig::ApplyConfig(UMediaPlayer& InMediaPlayer) const
{
	InMediaPlayer.PlayOnOpen = bPlayOnOpen ? 1 : 0;

	auto SetTrack = [&InMediaPlayer](EMediaPlayerTrack InTrack, int32 InTrackIndex)
		{
			if (const int32 TrackCount = InMediaPlayer.GetNumTracks(InTrack))
			{
				const int32 ClampedTrackIndex = FMath::Clamp(InTrackIndex, 0, TrackCount - 1);

				if (InMediaPlayer.GetSelectedTrack(InTrack) != ClampedTrackIndex)
				{
					InMediaPlayer.SelectTrack(InTrack, ClampedTrackIndex);
				}
			}
		};

	SetTrack(EMediaPlayerTrack::Video, TrackOptions.Video);
	SetTrack(EMediaPlayerTrack::Audio, TrackOptions.Audio);
	SetTrack(EMediaPlayerTrack::Subtitle, TrackOptions.Subtitle);
	SetTrack(EMediaPlayerTrack::Text, TrackOptions.Text);
	SetTrack(EMediaPlayerTrack::Caption, TrackOptions.Caption);
	SetTrack(EMediaPlayerTrack::Script, TrackOptions.Script);
	SetTrack(EMediaPlayerTrack::Metadata, TrackOptions.Metadata);

	if (InMediaPlayer.IsLooping() != bLooping)
	{
		InMediaPlayer.SetLooping(bLooping);
	}

	InMediaPlayer.SetNativeVolume(FMath::Clamp(Volume, 0.f, 1.f));

	if (PlaybackTimeRange.IsSet() && !FMath::IsNearlyZero(PlaybackTimeRange->Size()) && InMediaPlayer.SupportsPlaybackTimeRange())
	{
		const float DurationSeconds = static_cast<float>(InMediaPlayer.GetDuration().GetTotalSeconds());

		if (!FMath::IsNearlyZero(DurationSeconds))
		{
			const float ClampedMinRange = FMath::Clamp(PlaybackTimeRange->Min, 0.f, FMath::Min(DurationSeconds, PlaybackTimeRange->Max));
			const float ClampedMaxRange = FMath::Clamp(PlaybackTimeRange->Max, FMath::Min(DurationSeconds, PlaybackTimeRange->Min), DurationSeconds);

			if (FMath::IsNearlyEqual(ClampedMinRange, ClampedMaxRange))
			{
				InMediaPlayer.SetPlaybackTimeRange(FFloatInterval(ClampedMinRange, ClampedMaxRange));
			}
		}
	}

	InMediaPlayer.PlayOnOpen = bPlayOnOpen;

	if (InMediaPlayer.GetPlayerFacade()->ActivePlayerOptions.IsSet())
	{
		FMediaPlayerOptions& ActivePlayerOptions = InMediaPlayer.GetPlayerFacade()->ActivePlayerOptions.GetValue();

		ActivePlayerOptions.PlayOnOpen = bPlayOnOpen
			? EMediaPlayerOptionBooleanOverride::Enabled
			: EMediaPlayerOptionBooleanOverride::Disabled;

		ActivePlayerOptions.Loop = bLooping
			? EMediaPlayerOptionBooleanOverride::Enabled
			: EMediaPlayerOptionBooleanOverride::Disabled;
	}

	if (!bPlayOnOpen)
	{
		InMediaPlayer.SetRate(0.f);
	}
	else if (InMediaPlayer.GetRate() != Rate)
	{
		InMediaPlayer.Seek(0.f);
		InMediaPlayer.Play();
		ApplyRate(InMediaPlayer);
	}

	if (InMediaPlayer.GetTimeDelay().GetTotalSeconds() != TimeDelay)
	{
		InMediaPlayer.SetTimeDelay(TimeDelay);
	}

	if (InMediaPlayer.Shuffle != bShuffle)
	{
		InMediaPlayer.Shuffle = bShuffle ? 1 : 0;
	}

	InMediaPlayer.CacheAhead = CacheAhead;
	InMediaPlayer.CacheBehind = CacheBehind;
	InMediaPlayer.CacheBehindGame = CacheBehindGame;
}

bool FMediaStreamPlayerConfig::ApplyRate(UMediaPlayer& InMediaPlayer) const
{
 	bool bFoundRate = false;
	float NearestRate = 0;
	TArray<FFloatRange> SupportedRatesList;
	InMediaPlayer.GetSupportedRates(SupportedRatesList, /* Unthinned / no frames drops */ false);

	for (const FFloatRange& SupportedRates : SupportedRatesList)
	{
		if (SupportedRates.Contains(Rate))
		{
			NearestRate = Rate;
			bFoundRate = true;
			break;
		}

		if (Rate < SupportedRates.GetLowerBoundValue())
		{
			const float BelowRange = SupportedRates.GetLowerBoundValue() - Rate;

			if (!bFoundRate || BelowRange < FMath::Abs(Rate - NearestRate))
			{
				NearestRate = SupportedRates.GetLowerBoundValue();
				bFoundRate = true;
			}
		}
		else if (Rate > SupportedRates.GetUpperBoundValue())
		{
			const float AboveRange = Rate - SupportedRates.GetUpperBoundValue();

			if (!bFoundRate || AboveRange < FMath::Abs(Rate - NearestRate))
			{
				NearestRate = SupportedRates.GetUpperBoundValue();
				bFoundRate = true;
			}
		}
	}

	if (bFoundRate)
	{
		return InMediaPlayer.SetRate(NearestRate);
	}

	return false;
}
