// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMediaStreamObjectHandler.h"

#include "IMediaStreamSchemeHandler.h"

#if WITH_EDITOR
#include "MediaStreamSchemeHandlerManager.h"
#include "MediaStream.h"
#include "MediaStreamSource.h"
#include "PropertyHandle.h"
#endif

FMediaStreamObjectHandlerCreatePlayerParams FMediaStreamSchemeHandlerCreatePlayerParams::operator<<(UObject* InMediaSource) const
{
	FMediaStreamObjectHandlerCreatePlayerParams NewParams;
	NewParams.MediaStream = MediaStream;
	NewParams.Source = InMediaSource;
	NewParams.CurrentPlayer = CurrentPlayer;
	NewParams.bCanOpenSource = bCanOpenSource;

	return NewParams;
}

#if WITH_EDITOR
FMediaStreamSource* FMediaStreamSchemeHandlerLibrary::GetStreamSourcePtr(TWeakPtr<IPropertyHandle> InPropertyHandleWeak)
{
	TSharedPtr<IPropertyHandle> PropertyHandle = InPropertyHandleWeak.Pin();

	if (!PropertyHandle.IsValid())
	{
		return nullptr;
	}

	FProperty* Property = PropertyHandle->GetProperty();

	if (!Property)
	{
		return nullptr;
	}

	TArray<UObject*> Outers;
	PropertyHandle->GetOuterObjects(Outers);

	if (Outers.IsEmpty())
	{
		return nullptr;
	}

	return Property->ContainerPtrToValuePtr<FMediaStreamSource>(Outers[0]);
}

FName FMediaStreamSchemeHandlerLibrary::GetScheme(TWeakPtr<IPropertyHandle> InPropertyHandleWeak)
{
	if (TSharedPtr<IPropertyHandle> PropertyHandle = InPropertyHandleWeak.Pin())
	{
		if (FMediaStreamSource* StreamSource = GetStreamSourcePtr(PropertyHandle.ToSharedRef()))
		{
			return StreamSource->Scheme;
		}
	}

	return NAME_None;
}

FName FMediaStreamSchemeHandlerLibrary::GetScheme(UMediaStream* InMediaStream)
{
	if (InMediaStream)
	{
		return InMediaStream->GetSource().Scheme;
	}

	return NAME_None;
}

FName FMediaStreamSchemeHandlerLibrary::GetScheme(TWeakObjectPtr<UMediaStream> InMediaStreamWeak)
{
	return GetScheme(InMediaStreamWeak.Get());
}

FString FMediaStreamSchemeHandlerLibrary::GetPath(TWeakPtr<IPropertyHandle> InPropertyHandleWeak)
{
	if (TSharedPtr<IPropertyHandle> PropertyHandle = InPropertyHandleWeak.Pin())
	{
		if (FMediaStreamSource* StreamSource = GetStreamSourcePtr(PropertyHandle.ToSharedRef()))
		{
			return StreamSource->Path;
		}
	}

	return FString();
}

FString FMediaStreamSchemeHandlerLibrary::GetPath(UMediaStream* InMediaStream)
{
	if (InMediaStream)
	{
		return InMediaStream->GetSource().Path;
	}

	return FString();
}

FString FMediaStreamSchemeHandlerLibrary::GetPath(TWeakObjectPtr<UMediaStream> InMediaStreamWeak)
{
	return GetPath(InMediaStreamWeak.Get());
}

void FMediaStreamSchemeHandlerLibrary::SetSource(TSharedRef<IPropertyHandle> InPropertyHandle, const FName& InScheme, const FString& InPath)
{
	FMediaStreamSource* StreamSource = GetStreamSourcePtr(InPropertyHandle);

	if (!StreamSource)
	{
		return;
	}

	FProperty* Property = InPropertyHandle->GetProperty();
	TArray<UObject*> Outers;
	InPropertyHandle->GetOuterObjects(Outers);

	if (Outers.Num() != 1)
	{
		return;
	}

	Outers[0]->PreEditChange(Property);

	*StreamSource = FMediaStreamSchemeHandlerManager::Get().CreateSource(Outers[0], InScheme, InPath);

	FPropertyChangedEvent Event(Property, EPropertyChangeType::Interactive, Outers);
	Outers[0]->PostEditChangeProperty(Event);
}

void FMediaStreamSchemeHandlerLibrary::SetSource(UMediaStream* InMediaStream, const FName& InScheme, const FString& InPath)
{
	if (!IsValid(InMediaStream))
	{
		return;
	}

	FProperty* Property = InMediaStream->GetClass()->FindPropertyByName(UMediaStream::GetSourcePropertyName());
	UObject* Outer = InMediaStream->GetOuter();

	InMediaStream->PreEditChange(Property);

	if (Outer)
	{
		Outer->PreEditChange(Property);
	}

	InMediaStream->SetSource(FMediaStreamSchemeHandlerManager::Get().CreateSource(InMediaStream, InScheme, InPath));

	FPropertyChangedEvent Event(Property, EPropertyChangeType::Interactive, {InMediaStream});
	InMediaStream->PostEditChangeProperty(Event);

	if (Outer)
	{
		Outer->PostEditChangeProperty(Event);
	}
}

void FMediaStreamSchemeHandlerLibrary::SetSource(TWeakObjectPtr<UMediaStream> InMediaStreamWeak, const FName& InScheme, const FString& InPath)
{
	SetSource(InMediaStreamWeak.Get(), InScheme, InPath);
}

#endif
