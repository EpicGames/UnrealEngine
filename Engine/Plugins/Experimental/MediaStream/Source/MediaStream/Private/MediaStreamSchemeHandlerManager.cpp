// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaStreamSchemeHandlerManager.h"

#include "Engine/Engine.h"
#include "IMediaStreamSchemeHandler.h"
#include "MediaStream.h"
#include "MediaStreamModule.h"
#include "MediaStreamSource.h"

FMediaStreamSchemeHandlerManager& FMediaStreamSchemeHandlerManager::Get()
{
	static FMediaStreamSchemeHandlerManager Manager;
	return Manager;
}

bool FMediaStreamSchemeHandlerManager::HasSchemeHandler(FName InScheme) const
{
	return Handlers.Contains(InScheme);
}

TArray<FName> FMediaStreamSchemeHandlerManager::GetSchemeHandlerNames() const
{
	TArray<FName> Keys;
	Handlers.GenerateKeyArray(Keys);

	return Keys;
}

bool FMediaStreamSchemeHandlerManager::RegisterSchemeHandler(FName InScheme, const TSharedRef<IMediaStreamSchemeHandler>& InHandler)
{
	if (Handlers.Contains(InScheme))
	{
		return false;
	}

	Handlers.Add(InScheme, InHandler);

	return true;
}

TSharedPtr<IMediaStreamSchemeHandler> FMediaStreamSchemeHandlerManager::UnregisterSchemeHandler(FName InScheme)
{
	if (const TSharedRef<IMediaStreamSchemeHandler>* HandlerPtr = Handlers.Find(InScheme))
	{
		TSharedRef<IMediaStreamSchemeHandler> Handler = *HandlerPtr;
		Handlers.Remove(InScheme);

		return Handler;
	}

	return nullptr;
}

TSharedPtr<IMediaStreamSchemeHandler> FMediaStreamSchemeHandlerManager::GetHandlerTypeForScheme(FName InScheme) const
{
	if (InScheme.IsNone())
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Scheme in FMediaStreamSchemeHandlerManager::GetHandlerTypeForScheme");
		return nullptr;
	}

	if (const TSharedRef<IMediaStreamSchemeHandler>* HandlerPtr = Handlers.Find(InScheme))
	{
		return *HandlerPtr;
	}

	return nullptr;
}

UMediaPlayer* FMediaStreamSchemeHandlerManager::CreateOrUpdatePlayer(const FMediaStreamSchemeHandlerCreatePlayerParams& InParams) const
{
	const FMediaStreamSource& Source = InParams.MediaStream->GetSource();
	const TSharedRef<IMediaStreamSchemeHandler>* HandlerPtr = Handlers.Find(Source.Scheme);

	if (!HandlerPtr)
	{
		UE_LOGF(LogMediaStream, Error, "No handler for scheme in FMediaStreamSchemeHandlerManager::CreateOrUpdatePlayer [%ls]", *Source.Scheme.ToString());
		return nullptr;
	}

	return (*HandlerPtr)->CreateOrUpdatePlayer(InParams);
}

FMediaStreamSource FMediaStreamSchemeHandlerManager::CreateSource(UObject* InOuter, FName InScheme, const FString& InPath) const
{
	if (!IsValid(InOuter))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream in FMediaStreamSchemeHandlerManager::GetSourceFromSchemePath");
		return {};
	}

	const TSharedRef<IMediaStreamSchemeHandler>* HandlerPtr = Handlers.Find(InScheme);

	if (!HandlerPtr)
	{
		UE_LOGF(LogMediaStream, Error, "Missing handler for Scheme in FMediaStreamSchemeHandlerManager::GetMediaSourceFromSchemePath [%ls://%ls]", *InScheme.ToString(), *InPath);
		return {};
	}

	return (*HandlerPtr)->CreateSource(InOuter, InPath);
}
