// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaStreamSchemeHandlerSubsystem.h"

#include "Engine/Engine.h"
#include "MediaStreamModule.h"
#include "MediaStreamSchemeHandlerManager.h"
#include "MediaStreamSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaStreamSchemeHandlerSubsystem)

UMediaStreamSchemeHandlerSubsystem* UMediaStreamSchemeHandlerSubsystem::Get()
{
	if (!UObjectInitialized())
	{
		UE_LOGF(LogMediaStream, Error, "UObject system not initialized in UMediaStreamSchemeHandlerSubsystem::Get");
		return nullptr;
	}

	if (!GEngine)
	{
		UE_LOGF(LogMediaStream, Error, "Invalid GEngine in UMediaStreamSchemeHandlerSubsystem::Get");
		return nullptr;
	}

	UMediaStreamSchemeHandlerSubsystem* SchemeHandlerSubsystem = GEngine->GetEngineSubsystem<UMediaStreamSchemeHandlerSubsystem>();

	if (!SchemeHandlerSubsystem)
	{
		UE_LOGF(LogMediaStream, Error, "Unable to find Media Source Url Handler Subsystem in UMediaStreamSchemeHandlerSubsystem::Get");
		return nullptr;
	}

	return SchemeHandlerSubsystem;
}

bool UMediaStreamSchemeHandlerSubsystem::HasSchemeHandler(FName InScheme) const
{
	return FMediaStreamSchemeHandlerManager::Get().HasSchemeHandler(InScheme);
}

TArray<FName> UMediaStreamSchemeHandlerSubsystem::GetSchemeHandlerNames() const
{
	return FMediaStreamSchemeHandlerManager::Get().GetSchemeHandlerNames();
}

UMediaPlayer* UMediaStreamSchemeHandlerSubsystem::CreateOrUpdatePlayer(const FMediaStreamSchemeHandlerCreatePlayerParams& InParams) const
{
	return FMediaStreamSchemeHandlerManager::Get().CreateOrUpdatePlayer(InParams);
}

FMediaStreamSource UMediaStreamSchemeHandlerSubsystem::CreateSource(UObject* InOuter, FName InScheme, const FString& InPath) const
{
	return FMediaStreamSchemeHandlerManager::Get().CreateSource(InOuter, InScheme, InPath);
}
