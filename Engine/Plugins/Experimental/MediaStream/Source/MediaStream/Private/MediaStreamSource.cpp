// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaStreamSource.h"

#include "MediaStreamSchemeHandlerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaStreamSource)

bool FMediaStreamSource::operator==(const FMediaStreamSource& InOther) const
{
	return Scheme == InOther.Scheme &&
		(Scheme.IsNone() || (Path.Equals(InOther.Path)));
}

TArray<FName> UMediaStreamSourceBlueprintFunctionLibrary::GetSchemeTypes()
{
	return FMediaStreamSchemeHandlerManager::Get().GetSchemeHandlerNames();
}
