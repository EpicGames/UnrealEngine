// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaStreamSourceBlueprintLibrary.h"

#include "MediaStream.h"
#include "MediaStreamModule.h"
#include "MediaStreamObjectHandlerManager.h"
#include "MediaStreamSchemeHandlerManager.h"
#include "SchemeHandlers/MediaStreamAssetSchemeHandler.h"
#include "SchemeHandlers/MediaStreamFileSchemeHandler.h"
#include "SchemeHandlers/MediaStreamManagedSchemeHandler.h"
#include "SchemeHandlers/MediaStreamSubobjectSchemeHandler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaStreamSourceBlueprintLibrary)

bool UMediaStreamSourceBlueprintLibrary::IsValidMediaSource(const FMediaStreamSource& InSource)
{
	return !InSource.Scheme.IsNone() && !InSource.Path.IsEmpty();
}

bool UMediaStreamSourceBlueprintLibrary::IsAssetValid(const TSoftObjectPtr<UObject>& InAsset)
{
	return IsAssetSoftPathValid(InAsset.ToSoftObjectPath());
}

bool UMediaStreamSourceBlueprintLibrary::IsAssetPathValid(const FString& InPath)
{
	return IsAssetSoftPathValid(FSoftObjectPath(InPath));
}

bool UMediaStreamSourceBlueprintLibrary::IsAssetSoftPathValid(const FSoftObjectPath& InPath)
{
	return !InPath.IsNull() && InPath.IsValid() && InPath.IsAsset();
}

FMediaStreamSource UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromSchemePath(UMediaStream* InMediaStream, FName InScheme, const FString& InPath)
{
	if (!IsValid(InMediaStream))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream in UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromSchemePath");
		return {};
	}

	if (InScheme.IsNone())
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Scheme in UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromSchemePath");
		return {};
	}

	if (InPath.IsEmpty())
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Path in UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromSchemePath");
		return {};
	}

	return FMediaStreamSchemeHandlerManager::Get().CreateSource(InMediaStream, InScheme, InPath);
}

FMediaStreamSource UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromAsset(UMediaStream* InMediaStream, const TSoftObjectPtr<UObject>& InObject)
{
	if (!IsValid(InMediaStream))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream in UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromAsset");
		return {};
	}

	if (InObject.IsNull())
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Asset in UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromAsset");
		return {};
	}

	return MakeMediaSourceFromSchemePath(InMediaStream, FMediaStreamAssetSchemeHandler::Scheme, InObject.ToSoftObjectPath().ToString());
}

FMediaStreamSource UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromStreamName(UMediaStream* InMediaStream, FName InStreamName)
{
	if (!IsValid(InMediaStream))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream in UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromStreamName");
		return {};
	}

	if (InStreamName.IsNone())
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Stream Name in UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromStreamName");
		return {};
	}

	return MakeMediaSourceFromSchemePath(InMediaStream, FMediaStreamManagedSchemeHandler::Scheme, InStreamName.GetPlainNameString());
}

FMediaStreamSource UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromFile(UMediaStream* InMediaStream, const FString& InFileName)
{
	if (!IsValid(InMediaStream))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream in UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromFile");
		return {};
	}

	if (InFileName.IsEmpty())
	{
		UE_LOGF(LogMediaStream, Error, "Invalid File Name in UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromFile");
		return {};
	}

	return MakeMediaSourceFromSchemePath(InMediaStream, FMediaStreamFileSchemeHandler::Scheme, InFileName);
}

FMediaStreamSource UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromSubobject(UMediaStream* InMediaStream, UObject* InObject)
{
	if (!IsValid(InMediaStream))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream in UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromObject");
		return {};
	}

	if (!IsValid(InObject))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Object in UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromObject");
		return {};
	}

	if (InObject->IsAsset())
	{
		UE_LOGF(LogMediaStream, Error, "Asset given instead of Subobject in UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromObject [%ls]", *InObject->GetPathName());
		return {};
	}

		const UClass* ObjectClass = InObject->GetClass();

	if (!FMediaStreamObjectHandlerManager::Get().CanHandleObject(ObjectClass))
	{
		UE_LOGF(LogMediaStream, Error, "No registered handler for class in UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromObject [%ls]", *ObjectClass->GetName());
		return {};
	}

	const FString FullPath = InObject->GetPathName();
	const FString RelativePath = InObject->GetPathName(InMediaStream);

	if (FullPath == RelativePath)
	{
		UE_LOGF(LogMediaStream, Error, "Subobject is not a descendant of the Root Object in UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromObject");
		return {};
	}

	return MakeMediaSourceFromSchemePath(InMediaStream, FMediaStreamSubobjectSchemeHandler::Scheme, InObject->GetName());
}

FMediaStreamSource UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromSubobjectClass(UMediaStream* InMediaStream, const UClass* InClass)
{
	if (!IsValid(InMediaStream))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream in UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromClass");
		return {};
	}

	if (!IsValid(InClass))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Class in UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromClass");
		return {};
	}

	if (!FMediaStreamObjectHandlerManager::Get().CanHandleObject(InClass))
	{
		UE_LOGF(LogMediaStream, Error, "No registered handler for class in UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromClass [%ls]", *InClass->GetName());
		return {};
	}

	UObject* Object = NewObject<UObject>(InMediaStream, InClass, NAME_None, RF_Transactional);

	return MakeMediaSourceFromSubobject(InMediaStream, Object);
}
