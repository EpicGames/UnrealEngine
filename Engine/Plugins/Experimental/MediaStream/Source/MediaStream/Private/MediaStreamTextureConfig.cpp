// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaStreamTextureConfig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaStreamTextureConfig)

bool FMediaStreamTextureConfig::operator==(const FMediaStreamTextureConfig& InOther) const
{
	return bEnableMipGen == InOther.bEnableMipGen;
}

void FMediaStreamTextureConfig::ApplyConfig(UMediaTexture& InMediaTexture) const
{
	if (InMediaTexture.EnableGenMips == bEnableMipGen)
	{
		return;
	}

	InMediaTexture.EnableGenMips = bEnableMipGen;
	InMediaTexture.UpdateResource();
}
