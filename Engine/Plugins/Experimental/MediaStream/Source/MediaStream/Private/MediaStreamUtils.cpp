// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaStreamUtils.h"

#include "IMediaStreamPlayer.h"
#include "MediaStream.h"
#include "MediaStreamModule.h"
#include "MediaTexture.h"

namespace UE::MediaStream::Utils
{
	FMediaPlayerOptions MakePlayerOptions(const UMediaStream* InMediaStream)
	{
		if (InMediaStream)
		{
			if (const IMediaStreamPlayer* MediaStreamPlayer = InMediaStream->GetPlayer().GetInterface())
			{
				// Always provide the media texture as an extra option. 
				TMap<FName, FVariant> ExtraOptions;
				if (UMediaTexture* ViewTexture = MediaStreamPlayer->GetMediaTexture())
				{
					ExtraOptions.Emplace(MediaPlayerOptionValues::ViewMediaTexture(), FVariant(FSoftObjectPath(ViewTexture).ToString()));
				}
				return MediaStreamPlayer->GetPlayerConfig().CreateOptions(MediaStreamPlayer->GetRequestedSeekTime(), ExtraOptions);
			}
		}

		FMediaPlayerOptions Options;
		Options.SetAllAsOptional();
		return Options;
	}
	
	void BindMediaTextureToPlayer(const UMediaStream* InMediaStream, UMediaPlayer* InMediaPlayer)
	{
		// A null player would silently unbind the texture (UMediaTexture::SetMediaPlayer treats
		// nullptr as detach). All in-tree callers pass a valid player; treat null as a programming
		// error so it surfaces in dev builds, then fall through (the SetMediaPlayer call below is
		// still safe).
		ensureMsgf(InMediaPlayer != nullptr,
			TEXT("UE::MediaStream::Utils::BindMediaTextureToPlayer called with a null player; the texture's existing player will be detached."));

		if (InMediaStream)
		{
			if (const IMediaStreamPlayer* MediaStreamPlayer = InMediaStream->GetPlayer().GetInterface())
			{
				if (UMediaTexture* MediaTexture = MediaStreamPlayer->GetMediaTexture())
				{
					MediaTexture->SetMediaPlayer(InMediaPlayer);
				}
			}
		}
	}
}