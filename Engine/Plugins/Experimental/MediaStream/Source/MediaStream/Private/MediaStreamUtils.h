// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaPlayerOptions.h"

class UMediaPlayer;
class UMediaStream;

namespace UE::MediaStream::Utils
{
	/** Build player options from the stream's config; injects a ViewMediaTexture entry pointing at the stream's media texture when available. */
	FMediaPlayerOptions MakePlayerOptions(const UMediaStream* InMediaStream);

	/**
	 * Assign the media player to the current media stream's media texture.
	 * @param InMediaStream Media stream to get the media texture from.
	 * @param InMediaPlayer Media player to bind to the media texture.
	 */
	void BindMediaTextureToPlayer(const UMediaStream* InMediaStream, UMediaPlayer* InMediaPlayer);
}