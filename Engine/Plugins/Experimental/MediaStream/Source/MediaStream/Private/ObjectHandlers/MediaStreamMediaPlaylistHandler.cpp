// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectHandlers/MediaStreamMediaPlaylistHandler.h"

#include "MediaPlayer.h"
#include "MediaPlaylist.h"
#include "MediaStream.h"
#include "MediaStreamModule.h"
#include "MediaStreamUtils.h"

UClass* FMediaStreamMediaPlaylistHandler::GetClass()
{
	return UMediaPlaylist::StaticClass();
}

UMediaPlayer* FMediaStreamMediaPlaylistHandler::CreateOrUpdatePlayer(const FMediaStreamObjectHandlerCreatePlayerParams& InParams)
{
	if (!InParams.MediaStream)
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream in FMediaStreamMediaPlaylistHandler::CreateOrUpdatePlayer");
		return nullptr;
	}

	if (!InParams.Source)
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Source Object in FMediaStreamMediaPlaylistHandler::CreateOrUpdatePlayer");
		return nullptr;
	}

	UMediaPlaylist* MediaPlaylist = Cast<UMediaPlaylist>(InParams.Source);

	if (!MediaPlaylist)
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Playlist in FMediaStreamMediaPlaylistHandler::CreateOrUpdatePlayer");
		return nullptr;
	}

	if (InParams.CurrentPlayer)
	{
		UMediaSource* FirstPlaylistEntry = MediaPlaylist->Get(0);

		if (InParams.bCanOpenSource)
		{
			// For OpenPlaylist, there is no variant of the function where we can give the player options,
			// so we bind the texture to the player instead.
			UE::MediaStream::Utils::BindMediaTextureToPlayer(InParams.MediaStream, InParams.CurrentPlayer);
		}

		const bool bIsValidPlayer = InParams.bCanOpenSource
			? InParams.CurrentPlayer->OpenPlaylist(MediaPlaylist)
			: (!FirstPlaylistEntry || InParams.CurrentPlayer->CanPlaySource(FirstPlaylistEntry));

		if (bIsValidPlayer)
		{
			return InParams.CurrentPlayer;
		}
	}

	if (!InParams.bCanOpenSource)
	{
		UE_LOGF(LogMediaStream, Error, "Cannot create new player at the moment in FMediaStreamMediaPlaylistHandler::CreateOrUpdatePlayer");
		return nullptr;
	}

	UMediaPlayer* MediaPlayer = NewObject<UMediaPlayer>(InParams.MediaStream);

	if (!IsRunningCommandlet())
	{
		// For OpenPlaylist, there is no variant of the function where we can give the player options,
		// so we bind the texture to the player instead.
		UE::MediaStream::Utils::BindMediaTextureToPlayer(InParams.MediaStream, MediaPlayer);

		if (!MediaPlayer->OpenPlaylist(MediaPlaylist))
		{
			UE_LOGF(LogMediaStream, Error, "Unable to create player for Media Playlist in FMediaStreamMediaPlaylistHandler::CreateOrUpdatePlayer");
			return nullptr;
		}
	}

	return MediaPlayer;
}
