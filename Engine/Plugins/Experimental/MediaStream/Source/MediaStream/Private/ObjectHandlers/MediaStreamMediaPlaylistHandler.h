// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaStreamObjectHandler.h"

class FMediaStreamMediaPlaylistHandler : public IMediaStreamObjectHandler
{
public:
	static UClass* GetClass();

	//~ Begin IMediaStreamObjectHandler
	virtual UMediaPlayer* CreateOrUpdatePlayer(const FMediaStreamObjectHandlerCreatePlayerParams& InParams) override;
	//~ End IMediaStreamObjectHandler
};
