// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectHandlers/MediaStreamMediaSourceHandler.h"

#include "IMediaStreamPlayer.h"
#include "MediaPlayer.h"
#include "MediaSource.h"
#include "MediaStream.h"
#include "MediaStreamModule.h"
#include "MediaStreamUtils.h"

UClass* FMediaStreamMediaSourceHandler::GetClass()
{
	return UMediaSource::StaticClass();
}

UMediaPlayer* FMediaStreamMediaSourceHandler::CreateOrUpdatePlayer(const FMediaStreamObjectHandlerCreatePlayerParams& InParams)
{
	if (!InParams.MediaStream)
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream in FMediaStreamMediaSourceHandler::CreateOrUpdatePlayer");
		return nullptr;
	}

	if (!InParams.Source)
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Source Object in FMediaStreamMediaSourceHandler::CreateOrUpdatePlayer");
		return nullptr;
	}

	IMediaStreamPlayer* MediaStreamPlayer = InParams.MediaStream->GetPlayer().GetInterface();

	if (!MediaStreamPlayer)
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream Player in FMediaStreamMediaSourceHandler::CreateOrUpdatePlayer");
		return nullptr;
	}

	UMediaSource* MediaSource = Cast<UMediaSource>(InParams.Source);

	if (!MediaSource)
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Source in FMediaStreamMediaSourceHandler::CreateOrUpdatePlayer");
		return nullptr;
	}

	if (InParams.CurrentPlayer)
	{
		if (InParams.bCanOpenSource)
		{
			// Bind the media texture to the player up-front so it has a video sink for proper track init.
			UE::MediaStream::Utils::BindMediaTextureToPlayer(InParams.MediaStream, InParams.CurrentPlayer);
		}

		const bool bIsValidPlayer = InParams.bCanOpenSource
			? InParams.CurrentPlayer->OpenSourceWithOptions(MediaSource, UE::MediaStream::Utils::MakePlayerOptions(InParams.MediaStream))
			: InParams.CurrentPlayer->CanPlaySource(MediaSource);

		if (bIsValidPlayer)
		{
			return InParams.CurrentPlayer;
		}
	}

	if (!InParams.bCanOpenSource)
	{
		UE_LOGF(LogMediaStream, Error, "Cannot create new player at the moment in FMediaStreamMediaSourceHandler::CreateOrUpdatePlayer");
		return nullptr;
	}

	UMediaPlayer* MediaPlayer = NewObject<UMediaPlayer>(InParams.MediaStream, NAME_None, RF_Transactional);

	if (!IsRunningCommandlet())
	{
		// Bind the media texture to the player up-front so it has a video sink for proper track init.
		UE::MediaStream::Utils::BindMediaTextureToPlayer(InParams.MediaStream, MediaPlayer);

		if (!MediaPlayer->OpenSourceWithOptions(MediaSource, UE::MediaStream::Utils::MakePlayerOptions(InParams.MediaStream)))
		{
			UE_LOGF(LogMediaStream, Error, "Unable to create player for Media Source in FMediaStreamMediaSourceHandler::CreateOrUpdatePlayer");
			return nullptr;
		}
	}

	return MediaPlayer;
}
