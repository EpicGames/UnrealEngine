// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectHandlers/MediaStreamMediaStreamHandler.h"

#include "IMediaStreamPlayer.h"
#include "MediaPlayer.h"
#include "MediaStream.h"
#include "MediaStreamModule.h"

UClass* FMediaStreamMediaStreamHandler::GetClass()
{
	return UMediaStream::StaticClass();
}

UMediaPlayer* FMediaStreamMediaStreamHandler::CreateOrUpdatePlayer(const FMediaStreamObjectHandlerCreatePlayerParams& InParams)
{
	if (!InParams.MediaStream)
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream in FMediaStreamMediaStreamHandler::CreateOrUpdatePlayer");
		return nullptr;
	}

	if (!InParams.Source)
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Source Object in FMediaStreamMediaStreamHandler::CreateOrUpdatePlayer");
		return nullptr;
	}

	UMediaStream* MediaStream = Cast<UMediaStream>(InParams.Source);

	if (!MediaStream)
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream in FMediaStreamMediaStreamHandler::CreateOrUpdatePlayer");
		return nullptr;
	}

	IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface();

	if (!MediaStreamPlayer)
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream Player in FMediaStreamMediaStreamHandler::CreateOrUpdatePlayer");
		return nullptr;
	}

	UMediaPlayer* MediaPlayer = MediaStreamPlayer->GetPlayer();

	if (!MediaPlayer)
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Player FMediaStreamMediaStreamHandler::CreateOrUpdatePlayer");
		return nullptr;
	}

	return MediaPlayer;
}
