// Copyright Epic Games, Inc. All Rights Reserved.

#include "Players/MediaStreamLocalPlayer.h"

#include "CoreGlobals.h"
#include "IMediaClock.h"
#include "IMediaClockSink.h"
#include "IMediaModule.h"
#include "IMediaStreamSchemeHandler.h"
#include "LevelSequence.h"
#include "MediaPlaylist.h"
#include "MediaStream.h"
#include "MediaStreamModule.h"
#include "MediaStreamSchemeHandlerManager.h"
#include "MediaTexture.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "MovieSceneMediaSection.h"
#include "MovieSceneTrack.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaStreamLocalPlayer)

/**
 * MediaClockSink Callback utility class for the MediaStreamLocalPlayer. 
 */
class FMediaStreamLocalPlayerClockSink : public IMediaClockSink
{
public:
	FMediaStreamLocalPlayerClockSink(UMediaStreamLocalPlayer& InPlayer)
		: PlayerWeak(&InPlayer)
	{ }

	virtual ~FMediaStreamLocalPlayerClockSink() override = default;

	/** Returns the media module's media clock. */
	static IMediaClock* GetMediaClock()
	{
		if (IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media"))
		{
			return &MediaModule->GetClock();
		}
		return nullptr;
	}

	/** Register the clock sink with media module. */
	static void RegisterClockSink(UMediaStreamLocalPlayer& InPlayer)
	{
		if (!InPlayer.ClockSink.IsValid())
		{
			if (IMediaClock* MediaClock = GetMediaClock())
			{
				InPlayer.ClockSink = MakeShared<FMediaStreamLocalPlayerClockSink>(InPlayer);
				MediaClock->AddSink(InPlayer.ClockSink.ToSharedRef());
			}
		}
	}

	/** Unregister the clock sink from media module. */
	static void UnregisterClockSink(UMediaStreamLocalPlayer& InPlayer)
	{
		if (InPlayer.ClockSink.IsValid())
		{
			if (IMediaClock* MediaClock = GetMediaClock())
			{
				MediaClock->RemoveSink(InPlayer.ClockSink.ToSharedRef());
			}
			InPlayer.ClockSink.Reset();
		}
	}
	
	//~ Begin IMediaClockSink
	virtual void TickOutput(FTimespan InDeltaTime, FTimespan InTimecode) override
	{
		if (UMediaStreamLocalPlayer* Player = PlayerWeak.Get())
		{
			Player->Initialize();
			
			// Because this callback is only to delay player initialization, we can remove it when it is done.
			UnregisterClockSink(*Player);
		}
	}
	//~ End IMediaClockSink

private:
	TWeakObjectPtr<UMediaStreamLocalPlayer> PlayerWeak;
};

UMediaStreamLocalPlayer::~UMediaStreamLocalPlayer()
{
	if (!UObjectInitialized())
	{
		return;
	}

	DeinitPlayer();
}

UMediaStream* UMediaStreamLocalPlayer::GetMediaStream() const
{
	if (UObjectInitialized())
	{
		return Cast<UMediaStream>(GetOuter());
	}

	return nullptr;
}

bool UMediaStreamLocalPlayer::IsReadOnly() const
{
	return false;
}

void UMediaStreamLocalPlayer::OnCreated()
{
	MediaTexture = NewObject<UMediaTexture>(this);
	InitTexture();
}

void UMediaStreamLocalPlayer::Deinitialize()
{
	DeinitPlayer();
	DeinitTexture();
}

#if WITH_EDITOR
void UMediaStreamLocalPlayer::PreEditUndo()
{
	Super::PreEditUndo();

	TextureConfig_PreUndo = TextureConfig;
	PlayerConfig_PreUndo = PlayerConfig;
}

void UMediaStreamLocalPlayer::PostEditUndo()
{
	Super::PostEditUndo();

	if (TextureConfig_PreUndo != TextureConfig)
	{
		ApplyTextureConfig();
	}

	if (PlayerConfig_PreUndo != PlayerConfig)
	{
		ApplyPlayerConfig();
	}
}

void UMediaStreamLocalPlayer::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	static const FName PlayerConfigName = GET_MEMBER_NAME_CHECKED(UMediaStreamLocalPlayer, PlayerConfig);
	static const FName TextureConfigName = GET_MEMBER_NAME_CHECKED(UMediaStreamLocalPlayer, TextureConfig);
	static const FName RequestedSeekTimeName = GET_MEMBER_NAME_CHECKED(UMediaStreamLocalPlayer, RequestedSeekFrame);
	static const FName PlaybackStateName = GET_MEMBER_NAME_CHECKED(UMediaStreamLocalPlayer, PlaybackState);
	static const FName PlaylistIndexName = GET_MEMBER_NAME_CHECKED(UMediaStreamLocalPlayer, PlaylistIndex);

	const FName PropertyName = InPropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == PlayerConfigName)
	{
		ApplyPlayerConfig();
	}
	else if (PropertyName == TextureConfigName)
	{
		ApplyTextureConfig();
	}
	else if (PropertyName == RequestedSeekTimeName)
	{
		ApplyRequestedSeekFrame();
	}
	else if (PropertyName == PlaybackStateName)
	{
		ApplyPlaybackState();
	}
	else if (PropertyName == PlaylistIndexName)
	{
		ApplyPlaylistIndex();
	}
	
	if (UMediaStream* MediaStream = GetMediaStream())
	{
		MediaStream->GetOnPlayerChanged().Broadcast(MediaStream);
	}
}
#endif

void UMediaStreamLocalPlayer::PostDuplicate(bool bInDuplicateForPIE)
{
	Super::PostDuplicate(bInDuplicateForPIE);

#if WITH_EDITOR
	bIsPIE = bInDuplicateForPIE;
#endif

	// Delay initialization to the next game loop to let all media objects PostLoad.
	FMediaStreamLocalPlayerClockSink::RegisterClockSink(*this);
}

void UMediaStreamLocalPlayer::PostEditImport()
{
	Super::PostEditImport();

	// Delay initialization to the next game loop to let all media objects PostLoad.
	FMediaStreamLocalPlayerClockSink::RegisterClockSink(*this);
}

void UMediaStreamLocalPlayer::PostLoad()
{
	Super::PostLoad();

	bAllowOpenSource = FMediaStreamModule::Get().CanOpenSourceOnLoad();

	// Delay initialization to the next game loop to let all media objects PostLoad.
	FMediaStreamLocalPlayerClockSink::RegisterClockSink(*this);
}

void UMediaStreamLocalPlayer::PostNetReceive()
{
	Super::PostNetReceive();

	// Delay initialization to the next game loop to let all media objects PostLoad.
	FMediaStreamLocalPlayerClockSink::RegisterClockSink(*this);
}

void UMediaStreamLocalPlayer::BeginDestroy()
{
	Super::BeginDestroy();

	Deinitialize();
}

void UMediaStreamLocalPlayer::OnSourceChanged(const FMediaStreamSource& InSource)
{
	bPlayerNeedsUpdate = true;

	Initialize();
}

UMediaTexture* UMediaStreamLocalPlayer::GetMediaTexture() const
{
	return MediaTexture;
}

const FMediaStreamTextureConfig& UMediaStreamLocalPlayer::GetTextureConfig() const
{
	return TextureConfig;
}

void UMediaStreamLocalPlayer::SetTextureConfig(const FMediaStreamTextureConfig& InTextureConfig)
{
	TextureConfig = InTextureConfig;

	ApplyTextureConfig();
}

void UMediaStreamLocalPlayer::ApplyTextureConfig()
{
	if (IsValid(MediaTexture))
	{
		TextureConfig.ApplyConfig(*MediaTexture);
	}
}

bool UMediaStreamLocalPlayer::SetPlaylistIndex(int32 InIndex)
{
	PlaylistIndex = InIndex;

	return ApplyPlaylistIndex();
}

UMediaPlayer* UMediaStreamLocalPlayer::GetPlayer() const
{
	if (IsValid(Player.Get()))
	{
		return Player;
	}

	return nullptr;
}

bool UMediaStreamLocalPlayer::HasValidPlayer() const
{
	return IsValid(Player.Get());
}

const FMediaStreamPlayerConfig& UMediaStreamLocalPlayer::GetPlayerConfig() const
{
	return PlayerConfig;
}

void UMediaStreamLocalPlayer::SetPlayerConfig(const FMediaStreamPlayerConfig& InPlayerConfig)
{
	PlayerConfig = InPlayerConfig;

	ApplyPlayerConfig();
}

void UMediaStreamLocalPlayer::ApplyPlayerConfig()
{
	if (!IsValid(Player))
	{
		return;
	}

	// Stop media playing on open in the editor (only in PIE)
#if WITH_EDITOR
	const bool bOverridePlayOnOpen = HasAnyFlags(RF_WasLoaded) && !bIsPIE && !FMediaStreamModule::Get().CanAutoplay();
#else
	const bool bOverridePlayOnOpen = HasAnyFlags(RF_WasLoaded) && !FMediaStreamModule::Get().CanAutoplay();
#endif

	if (!bOverridePlayOnOpen)
	{
		PlayerConfig.ApplyConfig(*Player);
		return;
	}

	const bool bInitialPlayOnOpen = PlayerConfig.bPlayOnOpen;

	if (bOverridePlayOnOpen)
	{
		PlayerConfig.bPlayOnOpen = false;
	}

	PlayerConfig.ApplyConfig(*Player);

	if (bOverridePlayOnOpen)
	{
		PlayerConfig.bPlayOnOpen = bInitialPlayOnOpen;
	}
}

float UMediaStreamLocalPlayer::GetRequestedSeekTime() const
{
	if (!IsValid(Player))
	{
		return 0.f;
	}

	float FrameRate = Player->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);

	if (FMath::IsNearlyZero(FrameRate))
	{
		return 0.f;
	}

	return static_cast<float>(RequestedSeekFrame) / FrameRate;
}

bool UMediaStreamLocalPlayer::SetRequestedSeekTime(float InTime)
{
	if (!IsValid(Player))
	{
		return false;
	}

	float FrameRate = Player->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);

	if (FMath::IsNearlyZero(FrameRate))
	{
		return false;
	}

	RequestedSeekFrame = FMath::FloorToInt(InTime * FrameRate);

	return ApplyRequestedSeekFrame();
}

int32 UMediaStreamLocalPlayer::GetRequestedSeekFrame() const
{
	return RequestedSeekFrame;;
}

bool UMediaStreamLocalPlayer::SetRequestedSeekFrame(int32 InFrame)
{
	RequestedSeekFrame = InFrame;

	return ApplyRequestedSeekFrame();
}

bool UMediaStreamLocalPlayer::ApplyRequestedSeekFrame()
{
	bool bSuccess = false;

	if (RequestedSeekFrame >= 0 && IsValid(Player) && Player->SupportsSeeking())
	{
		const float FrameRate = Player->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);

		if (FrameRate > 0.f)
		{
			bSuccess = Player->Seek(FTimespan::FromSeconds(static_cast<float>(RequestedSeekFrame) / FrameRate));
		}
	}

	RequestedSeekFrame = INDEX_NONE;
	return bSuccess;
}

EMediaStreamPlaybackState UMediaStreamLocalPlayer::GetPlaybackState() const
{
	return PlaybackState;
}

bool UMediaStreamLocalPlayer::SetPlaybackState(EMediaStreamPlaybackState InState)
{
	PlaybackState = InState;

	return ApplyPlaybackState();
}

bool UMediaStreamLocalPlayer::ApplyPlaybackState()
{
	if (IsValid(Player))
	{
		switch (PlaybackState)
		{
			case EMediaStreamPlaybackState::Play:
				if (UMediaTimeStampInfo* TimeStamp = Player->GetTimeStamp())
				{
					const float FrameRate = Player->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);

					if (!FMath::IsNearlyZero(FrameRate))
					{
						const double TotalTime = Player->GetDuration().GetTotalSeconds();
						const double CurrentTime = TimeStamp->Time.GetTotalSeconds();

						if (!PlayerConfig.bLooping && FMath::Abs(TotalTime - CurrentTime) < (2.0 / FrameRate))
						{
							Player->Seek(0);
						}
					}
				}

				return PlayerConfig.ApplyRate(*Player);

			case EMediaStreamPlaybackState::Pause:
				return Player->SetRate(0.f);
		}
	}

	return false;
}

int32 UMediaStreamLocalPlayer::GetPlaylistIndex() const
{
	return PlaylistIndex;
}

bool UMediaStreamLocalPlayer::ApplyPlaylistIndex()
{
	if (IsValid(Player) && PlaylistIndex >= 0)
	{
		if (UMediaPlaylist* Playlist = Player->GetPlaylist())
		{
			if (Player->GetPlaylistIndex() != PlaylistIndex)
			{
				return Player->OpenPlaylistIndex(Playlist, PlaylistIndex);
			}

			// Already correct
			return true;
		}
	}

	return false;
}

int32 UMediaStreamLocalPlayer::GetPlaylistNum() const
{
	if (UMediaPlaylist* Playlist = Player->GetPlaylist())
	{
		return Playlist->Num();
	}

	return 0;
}

UMediaStream* UMediaStreamLocalPlayer::GetSourceStream() const
{
	return GetMediaStream();
}

bool UMediaStreamLocalPlayer::OpenSource()
{
	bAllowOpenSource = true;
	bPlayerNeedsUpdate = true;
	InitPlayer();

	return Player && !Player->IsClosed();
}

bool UMediaStreamLocalPlayer::Play()
{
	return SetPlaybackState(EMediaStreamPlaybackState::Play);
}

bool UMediaStreamLocalPlayer::Pause()
{
	return SetPlaybackState(EMediaStreamPlaybackState::Pause);
}

bool UMediaStreamLocalPlayer::Rewind()
{
	return SetRequestedSeekTime(0.f);
}

bool UMediaStreamLocalPlayer::FastForward()
{
	if (IsValid(Player))
	{
		return SetRequestedSeekTime(Player->GetDuration().GetTotalSeconds());
	}

	return false;
}

bool UMediaStreamLocalPlayer::Previous()
{
	const int32 PlaylistNum = GetPlaylistNum();

	if (PlaylistNum < 1)
	{
		return false;
	}

	if (PlaylistNum == 1)
	{
		return Rewind();
	}

	const int32 NextIndex = GetPlaylistIndex() - 1;

	if (NextIndex < 0)
	{
		if (GetPlayerConfig().bLooping)
		{
			return SetPlaylistIndex(PlaylistNum - 1);
		}

		return false;
	}

	return SetPlaylistIndex(NextIndex);
}

bool UMediaStreamLocalPlayer::Next()
{
	const int32 PlaylistNum = GetPlaylistNum();

	if (PlaylistNum < 1)
	{
		return false;
	}

	if (PlaylistNum == 1)
	{
		return Rewind();
	}

	const int32 NextIndex = GetPlaylistIndex() + 1;

	if (NextIndex >= PlaylistNum)
	{
		if (GetPlayerConfig().bLooping)
		{
			return SetPlaylistIndex(0);
		}

		return false;
	}

	return SetPlaylistIndex(NextIndex);
}

bool UMediaStreamLocalPlayer::Close()
{
	DeinitPlayer();
	return true;
}

void UMediaStreamLocalPlayer::Initialize()
{
	InitTexture();
	InitPlayer();
}

void UMediaStreamLocalPlayer::InitTexture()
{
	if (IsValid(MediaTexture))
	{
		ApplyTextureConfig();
	}
}

void UMediaStreamLocalPlayer::DeinitTexture()
{
	if (IsValid(MediaTexture))
	{
		const bool bIsPlayerValid = IsValid(Player);

		MediaTexture->SetMediaPlayer(bIsPlayerValid ? Player : nullptr);

#if WITH_EDITOR
		MediaTexture->SetDefaultMediaPlayer(bIsPlayerValid ? Player : nullptr);
#endif
	}
}

void UMediaStreamLocalPlayer::InitPlayer()
{
	UMediaPlayer* CurrentPlayer = Player.Get();

	if (bPlayerNeedsUpdate || !IsValid(Player))
	{
#if WITH_EDITOR
		Player = FMediaStreamSchemeHandlerManager::Get().CreateOrUpdatePlayer({GetMediaStream(), Player.Get(), bIsPIE || bAllowOpenSource});
#else
		Player = FMediaStreamSchemeHandlerManager::Get().CreateOrUpdatePlayer({GetMediaStream(), Player.Get(), bAllowOpenSource});
#endif
		bPlayerNeedsUpdate = false;
	}	

	if (CurrentPlayer != Player)
	{
		UpdateSequencesWithCurrentPlayer();
	}

	const bool bIsPlayerValid = IsValid(Player);

	if (IsValid(MediaTexture))
	{
#if WITH_EDITOR
		MediaTexture->SetDefaultMediaPlayer(bIsPlayerValid ? Player : nullptr);
#endif

		MediaTexture->UpdateResource();
	}

	if (bIsPlayerValid)
	{
		if (!Player->OnMediaOpened.Contains(this, GET_FUNCTION_NAME_CHECKED(UMediaStreamLocalPlayer, OnMediaOpened)))
		{
			Player->OnMediaOpened.AddDynamic(this, &UMediaStreamLocalPlayer::OnMediaOpened);
		}

#if WITH_EDITOR
		Player->AffectedByPIEHandling = false;
#endif

		ApplyPlayerConfig();
	}
}

void UMediaStreamLocalPlayer::DeinitPlayer()
{
	if (Player && Player->IsValidLowLevel() && IsValid(Player))
	{
		Player->Close();
	}
}

void UMediaStreamLocalPlayer::OnMediaOpened(FString InOpenedUrl)
{
	ApplyPlayerConfig();
}

void UMediaStreamLocalPlayer::UpdateSequencesWithCurrentPlayer()
{
	UMediaStream* MediaStream = GetMediaStream();

	if (!MediaStream)
	{
		return;
	}

	for (UMovieSceneMediaSection* Section : TObjectRange<UMovieSceneMediaSection>())
	{
		const FGuid& ProxySource = Section->GetMediaSourceProxy().GetGuid();

		if (!ProxySource.IsValid())
		{
			continue;
		}

		UMovieSceneTrack* Track = Cast<UMovieSceneTrack>(Section->GetOuter());

		if (!Track)
		{
			continue;
		}

		UMovieScene* MovieScene = Cast<UMovieScene>(Track->GetOuter());

		if (!MovieScene)
		{
			continue;
		}

		UMovieSceneSequence* MovieSequence = Cast<UMovieSceneSequence>(MovieScene->GetOuter());

		if (!MovieSequence)
		{
			continue;
		}

		UWorld* World = MovieSequence->GetWorld();

		if (!World)
		{
			continue;
		}

		const FMovieSceneBindingReferences* References = MovieSequence->GetBindingReferences();

		if (!References)
		{
			continue;
		}

		const UE::UniversalObjectLocator::FResolveParams ResolveParams(World);

		TArray<UObject*, TInlineAllocator<1>> BoundObject;
		References->ResolveBinding(ProxySource, ResolveParams, BoundObject);

		if (BoundObject.IsEmpty() || BoundObject[0] != MediaStream)
		{
			continue;
		}

		Section->bUseExternalMediaPlayer = true;
		Section->ExternalMediaPlayer = Player;		
		Section->TryModify();
	}
}
