// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaStreamPlayer.h"
#include "UObject/Object.h"

#include "Misc/Guid.h"
#include "UObject/WeakObjectPtr.h"

#include "MediaStreamLocalPlayer.generated.h"

class FMediaStreamLocalPlayerClockSink;

/**
 * Media Stream Local Player. Plays a media source using a UMediaPlayer.
 */
UCLASS(BlueprintType, Blueprintable, ClassGroup = "Media Stream", DefaultToInstanced)
class UMediaStreamLocalPlayer : public UObject, public IMediaStreamPlayer
{
	GENERATED_BODY()

public:
	UMediaStreamLocalPlayer() = default;
	virtual ~UMediaStreamLocalPlayer() override;

	UFUNCTION(BlueprintCallable, DisplayName = "Set Requested Seek Time", Category = "Media Stream|Player")
	void BP_SetRequestedSeekTime(float InSeekTime)
	{
		SetRequestedSeekTime(InSeekTime);
	}

	UFUNCTION(BlueprintCallable, DisplayName = "Set Requested Seek Frame", Category = "Media Stream|Player")
	void BP_SetRequestedSeekFrame(int32 InSeekFrame)
	{
		SetRequestedSeekFrame(InSeekFrame);
	}

	UFUNCTION(BlueprintCallable, DisplayName = "Set Playback State", Category = "Media Stream|Player")
	void BP_SetPlaybackState(EMediaStreamPlaybackState InState)
	{
		SetPlaybackState(InState);
	}

	UFUNCTION(BlueprintCallable, DisplayName = "Set Playlist Index", Category = "Media Stream|Player")
	void BP_SetPlaylistIndex(int32 InIndex)
	{
		SetPlaylistIndex(InIndex);
	}

	//~ Begin IMediaStreamPlayer
	virtual UMediaStream* GetMediaStream() const override;
	virtual bool IsReadOnly() const override;
	virtual void OnCreated() override;
	virtual void OnSourceChanged(const FMediaStreamSource& InSource) override;
	virtual UMediaTexture* GetMediaTexture() const override;
	virtual const FMediaStreamTextureConfig& GetTextureConfig() const override;
	virtual void SetTextureConfig(const FMediaStreamTextureConfig& InTextureConfig) override;
	virtual void ApplyTextureConfig() override;
	virtual UMediaPlayer* GetPlayer() const override;
	virtual bool HasValidPlayer() const override;
	virtual const FMediaStreamPlayerConfig& GetPlayerConfig() const override;
	virtual void SetPlayerConfig(const FMediaStreamPlayerConfig& InPlayerConfig) override;
	virtual void ApplyPlayerConfig() override;
	virtual bool SetPlaylistIndex(int32 InIndex) override;
	virtual float GetRequestedSeekTime() const override;
	virtual bool SetRequestedSeekTime(float InTime) override;
	virtual int32 GetRequestedSeekFrame() const override;
	virtual bool SetRequestedSeekFrame(int32 InFrame) override;
	virtual EMediaStreamPlaybackState GetPlaybackState() const override;
	virtual bool SetPlaybackState(EMediaStreamPlaybackState InState) override;
	virtual int32 GetPlaylistIndex() const override;
	virtual int32 GetPlaylistNum() const override;
	virtual UMediaStream* GetSourceStream() const override;
	virtual bool OpenSource() override;
	virtual bool Play() override;
	virtual bool Pause() override;
	virtual bool Rewind() override;
	virtual bool FastForward() override;
	virtual bool Previous() override;
	virtual bool Next() override;
	virtual bool Close() override;
	virtual void Deinitialize() override;
	//~ End IMediaStreamPlayer

	//~ Begin UObject
#if WITH_EDITOR
	virtual void PreEditUndo() override;
	virtual void PostEditUndo() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif

	virtual void PostDuplicate(bool bInDuplicateForPIE) override;
	virtual void PostEditImport() override;
	virtual void PostLoad() override;
	virtual void PostNetReceive() override;
	virtual void BeginDestroy() override;
	//~ End UObject

protected:
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Media Stream|Texture", meta = (AllowPrivateAccess = true))
	TObjectPtr<UMediaTexture> MediaTexture;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media Stream|Texture", meta = (AllowPrivateAccess = true))
	FMediaStreamTextureConfig TextureConfig;

	UPROPERTY(BlueprintReadOnly, Category = "Media Stream|Player", Transient, DuplicateTransient, TextExportTransient,
		meta = (AllowPrivateAccess = true))
	TObjectPtr<UMediaPlayer> Player = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media Stream|Player", meta = (AllowPrivateAccess = true))
	FMediaStreamPlayerConfig PlayerConfig;

	/** Attempts to seek to this frame. Any value below 0 is ignored. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter = BP_SetRequestedSeekFrame, Getter, BlueprintSetter = BP_SetRequestedSeekFrame,
		Category = "Media Stream|Player", Interp, meta = (AllowPrivateAccess = true))
	int32 RequestedSeekFrame = INDEX_NONE;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter = BP_SetPlaybackState, Getter, BlueprintSetter = BP_SetPlaybackState,
		Category = "Media Stream|Player", Interp, meta = (AllowPrivateAccess = true))
	EMediaStreamPlaybackState PlaybackState = EMediaStreamPlaybackState::Pause;

	/** The index playing in the playlist. -1 to ignore. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter = BP_SetPlaylistIndex, Getter, BlueprintSetter = BP_SetPlaylistIndex,
		Category = "Media Stream|Player", Interp, meta = (AllowPrivateAccess = true))
	int32 PlaylistIndex = INDEX_NONE;

	FMediaStreamTextureConfig TextureConfig_PreUndo;
	FMediaStreamPlayerConfig PlayerConfig_PreUndo;

	bool bPlayerNeedsUpdate = false;

	/**
	 * Determines whether local players can create or update media players when opening a source.
	 */
	bool bAllowOpenSource = true;

	#if WITH_EDITOR
	/**
	 * Whether this object was duplicated into a PIE level.
	 */
	bool bIsPIE = false;
#endif

	friend FMediaStreamLocalPlayerClockSink;
	
	/** 
	 * Holds our clock sink if available. 
	 */
	TSharedPtr<FMediaStreamLocalPlayerClockSink> ClockSink;

	/**
	 * Initializes all of the components of the Media Stream.
	 */
	void Initialize();

	/**
	 * Initializes the Media Texture
	 */
	void InitTexture();

	/**
	 * Deinitializes the Media Texture
	 */
	void DeinitTexture();

	/**
	 * Initializes the Media Player.
	 */
	void InitPlayer();

	/**
	 * Deinitializes the Media Player and clears the object.
	 */
	void DeinitPlayer();

	/**
	 * Attempts to set the seek time on the currently active Media Player.
	 * @return True if successful.
	 */
	bool ApplyRequestedSeekFrame();

	/**
	 * Will either set the current Rate from the player config or 0 for paused.
	 * The player will be checked for valid rate values. The nearest possible rate will be chosen.
	 * @return True if successful.
	 */
	bool ApplyPlaybackState();

	/**
	 * Attempts to set the playlist index on the Media Player.
	 * @return True if successful.
	 */
	bool ApplyPlaylistIndex();

	/**
	 * Called when the player's media has opened.
	 */
	UFUNCTION()
	void OnMediaOpened(FString InOpenedUrl);

	void UpdateSequencesWithCurrentPlayer();
};
