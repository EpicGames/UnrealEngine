// Copyright Epic Games, Inc. All Rights Reserved.

#include "Players/MediaStreamProxyPlayer.h"

#include "MediaStream.h"
#include "SchemeHandlers/MediaStreamAssetSchemeHandler.h"
#include "UObject/ScriptInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaStreamProxyPlayer)

UMediaStreamProxyPlayer::~UMediaStreamProxyPlayer()
{
	Deinitialize();
}

TSoftObjectPtr<UMediaStream> UMediaStreamProxyPlayer::GetProxyStreamSoft() const
{
	return ProxyStreamSoft;
}

UMediaStream* UMediaStreamProxyPlayer::GetSourceStream() const
{
	// Skip this check if it's already loaded.
	if (ProxyStream != nullptr && ProxyStream == ProxyStreamSoft.Get())
	{
		return ProxyStream;
	}

	ProxyStream = nullptr;

	if (!ProxyStreamSoft.IsNull())
	{
		ProxyStream = ProxyStreamSoft.LoadSynchronous();
	}

	return ProxyStream;
}

bool UMediaStreamProxyPlayer::IsReadOnly() const
{
	return bReadOnly;
}

void UMediaStreamProxyPlayer::SetReadOnly(bool bInReadOnly)
{
	bReadOnly = bInReadOnly;
}

UMediaStream* UMediaStreamProxyPlayer::GetMediaStream() const
{
	if (UObjectInitialized())
	{
		return Cast<UMediaStream>(GetOuter());
	}

	return nullptr;
}

void UMediaStreamProxyPlayer::Deinitialize()
{
	ProxyStreamSoft.Reset();
	ProxyStream = nullptr;
}

#if WITH_EDITOR
void UMediaStreamProxyPlayer::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (UMediaStream* MediaStream = GetMediaStream())
	{
		MediaStream->GetOnPlayerChanged().Broadcast(MediaStream);
	}
}
#endif

void UMediaStreamProxyPlayer::OnSourceChanged(const FMediaStreamSource& InSource)
{
	ProxyStreamSoft.Reset();
	ProxyStream = nullptr;

	if (InSource.Scheme == FMediaStreamAssetSchemeHandler::Scheme)
	{
		if (IsValid(InSource.Object) && InSource.Object->GetClass() == UMediaStream::StaticClass())
		{
			// Lets us "cast" this soft pointer.
			ProxyStreamSoft = Cast<UMediaStream>(InSource.Object);
		}
	}
}

UMediaTexture* UMediaStreamProxyPlayer::GetMediaTexture() const
{
	if (UMediaStream* MediaStream = GetSourceStream())
	{
		if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
		{
			return MediaStreamPlayer->GetMediaTexture();
		}
	}

	return nullptr;
}

const FMediaStreamTextureConfig& UMediaStreamProxyPlayer::GetTextureConfig() const
{
	if (UMediaStream* MediaStream = GetSourceStream())
	{
		if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
		{
			return MediaStreamPlayer->GetTextureConfig();
		}
	}

	static const FMediaStreamTextureConfig Config;

	return Config;
}

void UMediaStreamProxyPlayer::SetTextureConfig(const FMediaStreamTextureConfig& InTextureConfig)
{
	if (!bReadOnly)
	{
		if (UMediaStream* MediaStream = GetSourceStream())
		{
			if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
			{
				MediaStreamPlayer->SetTextureConfig(InTextureConfig);
			}
		}
	}
}

void UMediaStreamProxyPlayer::ApplyTextureConfig()
{
	if (!bReadOnly)
	{
		if (UMediaStream* MediaStream = GetSourceStream())
		{
			if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
			{
				MediaStreamPlayer->ApplyTextureConfig();
			}
		}
	}
}

void UMediaStreamProxyPlayer::OnCreated()
{
}

bool UMediaStreamProxyPlayer::SetPlaylistIndex(int32 InIndex)
{
	if (!bReadOnly)
	{
		if (UMediaStream* MediaStream = GetSourceStream())
		{
			if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
			{
				return MediaStreamPlayer->SetPlaylistIndex(InIndex);
			}
		}
	}

	return false;
}

UMediaPlayer* UMediaStreamProxyPlayer::GetPlayer() const
{
	if (UMediaStream* MediaStream = GetSourceStream())
	{
		if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
		{
			return MediaStreamPlayer->GetPlayer();
		}
	}

	return nullptr;
}

bool UMediaStreamProxyPlayer::HasValidPlayer() const
{
	if (UMediaStream* MediaStream = GetSourceStream())
	{
		if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
		{
			return MediaStreamPlayer->HasValidPlayer();
		}
	}

	return false;
}

const FMediaStreamPlayerConfig& UMediaStreamProxyPlayer::GetPlayerConfig() const
{
	if (UMediaStream* MediaStream = GetSourceStream())
	{
		if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
		{
			return MediaStreamPlayer->GetPlayerConfig();
		}
	}

	static const FMediaStreamPlayerConfig Config;

	return Config;
}

void UMediaStreamProxyPlayer::SetPlayerConfig(const FMediaStreamPlayerConfig& InPlayerConfig)
{
	if (!bReadOnly)
	{
		if (UMediaStream* MediaStream = GetSourceStream())
		{
			if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
			{
				return MediaStreamPlayer->SetPlayerConfig(InPlayerConfig);
			}
		}
	}
}

void UMediaStreamProxyPlayer::ApplyPlayerConfig()
{
	if (!bReadOnly)
	{
		if (UMediaStream* MediaStream = GetSourceStream())
		{
			if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
			{
				MediaStreamPlayer->ApplyPlayerConfig();
			}
		}
	}
}

float UMediaStreamProxyPlayer::GetRequestedSeekTime() const
{
	if (UMediaStream* MediaStream = GetSourceStream())
	{
		if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
		{
			return MediaStreamPlayer->GetRequestedSeekTime();
		}
	}

	return 0.f;
}

bool UMediaStreamProxyPlayer::SetRequestedSeekTime(float InTime)
{
	if (!bReadOnly)
	{
		if (UMediaStream* MediaStream = GetSourceStream())
		{
			if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
			{
				return MediaStreamPlayer->SetRequestedSeekTime(InTime);
			}
		}
	}

	return false;
}

int32 UMediaStreamProxyPlayer::GetRequestedSeekFrame() const
{
	if (UMediaStream* MediaStream = GetSourceStream())
	{
		if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
		{
			return MediaStreamPlayer->GetRequestedSeekFrame();
		}
	}

	return 0;
}

bool UMediaStreamProxyPlayer::SetRequestedSeekFrame(int32 InFrame)
{
	if (!bReadOnly)
	{
		if (UMediaStream* MediaStream = GetSourceStream())
		{
			if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
			{
				return MediaStreamPlayer->SetRequestedSeekFrame(InFrame);
			}
		}
	}

	return false;
}

EMediaStreamPlaybackState UMediaStreamProxyPlayer::GetPlaybackState() const
{
	if (UMediaStream* MediaStream = GetSourceStream())
	{
		if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
		{
			return MediaStreamPlayer->GetPlaybackState();
		}
	}

	return EMediaStreamPlaybackState::Play;
}

bool UMediaStreamProxyPlayer::SetPlaybackState(EMediaStreamPlaybackState InState)
{
	if (!bReadOnly)
	{
		if (UMediaStream* MediaStream = GetSourceStream())
		{
			if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
			{
				return MediaStreamPlayer->SetPlaybackState(InState);
			}
		}
	}

	return false;
}

int32 UMediaStreamProxyPlayer::GetPlaylistIndex() const
{
	if (UMediaStream* MediaStream = GetSourceStream())
	{
		if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
		{
			return MediaStreamPlayer->GetPlaylistIndex();
		}
	}

	return -1;
}

int32 UMediaStreamProxyPlayer::GetPlaylistNum() const
{
	if (UMediaStream* MediaStream = GetSourceStream())
	{
		if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
		{
			return MediaStreamPlayer->GetPlaylistNum();
		}
	}

	return -1;
}

bool UMediaStreamProxyPlayer::OpenSource()
{
	if (!bReadOnly)
	{
		if (UMediaStream* MediaStream = GetSourceStream())
		{
			if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
			{
				return MediaStreamPlayer->OpenSource();
			}
		}
	}

	return false;
}

bool UMediaStreamProxyPlayer::Play()
{
	if (!bReadOnly)
	{
		if (UMediaStream* MediaStream = GetSourceStream())
		{
			if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
			{
				return MediaStreamPlayer->Play();
			}
		}
	}

	return false;
}

bool UMediaStreamProxyPlayer::Pause()
{
	if (!bReadOnly)
	{
		if (UMediaStream* MediaStream = GetSourceStream())
		{
			if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
			{
				return MediaStreamPlayer->Pause();
			}
		}
	}

	return false;
}

bool UMediaStreamProxyPlayer::Rewind()
{
	if (!bReadOnly)
	{
		if (UMediaStream* MediaStream = GetSourceStream())
		{
			if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
			{
				return MediaStreamPlayer->Rewind();
			}
		}
	}

	return false;
}

bool UMediaStreamProxyPlayer::FastForward()
{
	if (!bReadOnly)
	{
		if (UMediaStream* MediaStream = GetSourceStream())
		{
			if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
			{
				return MediaStreamPlayer->FastForward();
			}
		}
	}

	return false;
}

bool UMediaStreamProxyPlayer::Previous()
{
	if (!bReadOnly)
	{
		if (UMediaStream* MediaStream = GetSourceStream())
		{
			if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
			{
				return MediaStreamPlayer->Previous();
			}
		}
	}

	return false;
}

bool UMediaStreamProxyPlayer::Next()
{
	if (!bReadOnly)
	{
		if (UMediaStream* MediaStream = GetSourceStream())
		{
			if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
			{
				return MediaStreamPlayer->Next();
			}
		}
	}

	return false;
}

bool UMediaStreamProxyPlayer::Close()
{
	if (!bReadOnly)
	{
		if (UMediaStream* MediaStream = GetSourceStream())
		{
			if (TScriptInterface<IMediaStreamPlayer> MediaStreamPlayer = MediaStream->GetPlayer())
			{
				return MediaStreamPlayer->Close();
			}
		}
	}

	return false;
}
