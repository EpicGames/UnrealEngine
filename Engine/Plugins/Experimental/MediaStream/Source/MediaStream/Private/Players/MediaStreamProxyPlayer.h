// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaStreamPlayer.h"
#include "UObject/Object.h"

#include "MediaStreamProxyPlayer.generated.h"

/**
 * Media Stream Proxy Player. Forwards the player and texture from another Media Stream.
 */
UCLASS(BlueprintType, Blueprintable, ClassGroup = "Media Stream", DefaultToInstanced)
class UMediaStreamProxyPlayer : public UObject, public IMediaStreamPlayer
{
	GENERATED_BODY()

public:
	virtual ~UMediaStreamProxyPlayer() override;

	UFUNCTION(Category = "Media Stream|Player")
	TSoftObjectPtr<UMediaStream> GetProxyStreamSoft() const;

	UFUNCTION(Category = "Media Stream|Texture")
	void SetReadOnly(bool bInReadOnly);

	//~ Begin IMediaStreamPlayer
	virtual UMediaStream* GetMediaStream() const override;
	virtual bool IsReadOnly() const override;
	virtual void OnCreated() override;
	virtual void OnSourceChanged(const FMediaStreamSource& InSource) override;
	virtual UMediaTexture* GetMediaTexture() const override;
	virtual const FMediaStreamTextureConfig& GetTextureConfig() const override;
	virtual void SetTextureConfig(const FMediaStreamTextureConfig& InTextureConfig) override;
	virtual void ApplyTextureConfig() override;
	virtual UMediaPlayer* GetPlayer() const override;
	virtual bool HasValidPlayer() const override;
	virtual const FMediaStreamPlayerConfig& GetPlayerConfig() const override;
	virtual void SetPlayerConfig(const FMediaStreamPlayerConfig& InPlayerConfig) override;
	virtual void ApplyPlayerConfig() override;
	virtual bool SetPlaylistIndex(int32 InIndex) override;
	virtual float GetRequestedSeekTime() const override;
	virtual bool SetRequestedSeekTime(float InTime) override;
	virtual int32 GetRequestedSeekFrame() const override;
	virtual bool SetRequestedSeekFrame(int32 InFrame) override;
	virtual EMediaStreamPlaybackState GetPlaybackState() const override;
	virtual bool SetPlaybackState(EMediaStreamPlaybackState InState) override;
	virtual int32 GetPlaylistIndex() const override;
	virtual int32 GetPlaylistNum() const override;
	virtual UMediaStream* GetSourceStream() const override;
	virtual bool OpenSource() override;
	virtual bool Play() override;
	virtual bool Pause() override;
	virtual bool Rewind() override;
	virtual bool FastForward() override;
	virtual bool Previous() override;
	virtual bool Next() override;
	virtual bool Close() override;
	virtual void Deinitialize() override;
	//~ End IMediaStreamPlayer

#if WITH_EDITOR
	//~ Begin UObject
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End UObject
#endif

protected:
	/** Soft pointer to the proxied stream. */
	UPROPERTY()
	TSoftObjectPtr<UMediaStream> ProxyStreamSoft;

	/** Hard reference to the proxied stream. */
	UPROPERTY(Transient)
	mutable TObjectPtr<UMediaStream> ProxyStream;

	/** If true, the proxied stream won't receive set calls. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media Stream|Player")
	bool bReadOnly = true;
};
