// Copyright Epic Games, Inc. All Rights Reserved.

#include "SchemeHandlers/MediaStreamAssetSchemeHandler.h"

#include "IMediaStreamObjectHandler.h"
#include "MediaSource.h"
#include "MediaStream.h"
#include "MediaStreamModule.h"
#include "MediaStreamObjectHandlerManager.h"
#include "MediaStreamSource.h"
#include "MediaStreamSourceBlueprintLibrary.h"

#if WITH_EDITOR
#include "PropertyCustomizationHelpers.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Widgets/Text/STextBlock.h"
#endif

#define LOCTEXT_NAMESPACE "MediaStreamAssetSchemeHandler"

const FLazyName FMediaStreamAssetSchemeHandler::Scheme = TEXT("Asset");

FMediaStreamSource FMediaStreamAssetSchemeHandler::CreateSource(UObject* InOuter, const FString& InPath)
{
	FMediaStreamSource Source;

	if (!IsValid(InOuter))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream in FMediaStreamAssetSchemeHandler::CreateSource");
		return Source;
	}

	Source.Scheme = Scheme;

	UObject* Asset = ResolveAsset(InPath);

	if (!IsValid(Asset))
	{
		return Source;
	}

	UMediaSource* MediaSource = Cast<UMediaSource>(Asset);

	if (!MediaSource)
	{
		return Source;
	}

	Source.Path = InPath;
	Source.Object = MediaSource;

	return Source;
}

UMediaPlayer* FMediaStreamAssetSchemeHandler::CreateOrUpdatePlayer(const FMediaStreamSchemeHandlerCreatePlayerParams& InParams)
{
	if (!IsValid(InParams.MediaStream))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream in FMediaStreamAssetSchemeHandler::CreateOrUpdatePlayer");
		return nullptr;
	}

	const FMediaStreamSource& Source = InParams.MediaStream->GetSource();

	if (!IsValid(Source.Object))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Asset in FMediaStreamAssetSchemeHandler::CreateOrUpdatePlayer");
		return nullptr;
	}

	return FMediaStreamObjectHandlerManager::Get().CreateOrUpdatePlayer(InParams << Source.Object);
}

UObject* FMediaStreamAssetSchemeHandler::ResolveAsset(const FString& InPath)
{
	FSoftObjectPath SoftPath = InPath;

	if (!UMediaStreamSourceBlueprintLibrary::IsAssetSoftPathValid(SoftPath))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Asset Path in FMediaStreamAssetSchemeHandler::ResolveAsset [%ls]", *InPath);
		return nullptr;
	}

	TSoftObjectPtr<UObject> SoftObjectPtr(SoftPath);
	UObject* Asset = SoftObjectPtr.LoadSynchronous();

	if (!IsValid(Asset))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Asset in FMediaStreamAssetSchemeHandler::ResolveAsset [%ls]", *InPath);
		return nullptr;
	}

	if (!FMediaStreamObjectHandlerManager::Get().CanHandleObject(Asset))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Asset Class in FMediaStreamAssetSchemeHandler::CreateSource [%ls]", *InPath);
		return nullptr;
	}

	return Asset;
}

#if WITH_EDITOR
void FMediaStreamAssetSchemeHandler::CreatePropertyCustomization(UMediaStream* InMediaStream, IMediaStreamSchemeHandler::FCustomWidgets& InOutCustomWidgets)
{
	AddAssetSelector(InMediaStream, InOutCustomWidgets);
}

void FMediaStreamAssetSchemeHandler::AddAssetSelector(UMediaStream* InMediaStream, IMediaStreamSchemeHandler::FCustomWidgets& InOutCustomWidgets)
{
	if (!IsValid(InMediaStream))
	{
		return;
	}

	TWeakObjectPtr<UMediaStream> MediaStreamWeak = InMediaStream;

	InOutCustomWidgets.CustomRows.Add({
		LOCTEXT("Asset", "Asset"),
		SNew(SObjectPropertyEntryBox)
		.ObjectPath_Static(&FMediaStreamSchemeHandlerLibrary::GetPath, MediaStreamWeak)
		.OnObjectChanged(this, &FMediaStreamAssetSchemeHandler::OnAssetSelected, MediaStreamWeak)
		.ThumbnailPool(UThumbnailManager::Get().GetSharedThumbnailPool())
		.DisplayThumbnail(true)
		.AllowedClass(UMediaSource::StaticClass()),
		/* Enabled */ true,
		TAttribute<EVisibility>::CreateSP(this, &FMediaStreamAssetSchemeHandler::GetAssetSelectorVisibility, MediaStreamWeak),
		FMediaStreamSource::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FMediaStreamSource, Object))
	});
}

EVisibility FMediaStreamAssetSchemeHandler::GetAssetSelectorVisibility(TWeakObjectPtr<UMediaStream> InMediaStreamWeak) const
{
	if (FMediaStreamSchemeHandlerLibrary::GetScheme(InMediaStreamWeak) == Scheme)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

void FMediaStreamAssetSchemeHandler::OnAssetSelected(const FAssetData& InAssetData, TWeakObjectPtr<UMediaStream> InMediaStreamWeak)
{
	FMediaStreamSchemeHandlerLibrary::SetSource(InMediaStreamWeak, Scheme, InAssetData.GetSoftObjectPath().ToString());
}
#endif

#undef LOCTEXT_NAMESPACE
