// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaStreamSchemeHandler.h"

class UMediaSource;

#if WITH_EDITOR
struct EVisibility;
struct FAssetData;
#endif

class FMediaStreamAssetSchemeHandler : public IMediaStreamSchemeHandler
{
public:
	static const FLazyName Scheme;

	//~ Begin IMediaStreamSchemeHandler
	virtual FMediaStreamSource CreateSource(UObject* InOuter, const FString& InPath) override;
	virtual UMediaPlayer* CreateOrUpdatePlayer(const FMediaStreamSchemeHandlerCreatePlayerParams& InParams) override;

#if WITH_EDITOR
	virtual void CreatePropertyCustomization(UMediaStream* InMediaStream, IMediaStreamSchemeHandler::FCustomWidgets& InOutCustomWidgets) override;
#endif
	//~ End IMediaStreamSchemeHandler

protected:
	UObject* ResolveAsset(const FString& InPath);

#if WITH_EDITOR
	void AddAssetSelector(UMediaStream* InMediaStream, IMediaStreamSchemeHandler::FCustomWidgets& InOutCustomWidgets);

	EVisibility GetAssetSelectorVisibility(TWeakObjectPtr<UMediaStream> InMediaStreamWeak) const;

	void OnAssetSelected(const FAssetData& InAssetData, TWeakObjectPtr<UMediaStream> InMediaStreamWeak);
#endif
};
