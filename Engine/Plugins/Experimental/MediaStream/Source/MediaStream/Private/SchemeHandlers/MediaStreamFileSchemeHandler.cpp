// Copyright Epic Games, Inc. All Rights Reserved.

#include "SchemeHandlers/MediaStreamFileSchemeHandler.h"

#include "HAL/FileManager.h"
#include "IMediaStreamObjectHandler.h"
#include "MediaSource.h"
#include "MediaStream.h"
#include "MediaStreamModule.h"
#include "MediaStreamObjectHandlerManager.h"
#include "MediaStreamSource.h"
#include "Misc/Paths.h"

#if WITH_EDITOR
#include "Widgets/Input/SFilePathPicker.h"
#include "Widgets/Text/STextBlock.h"
#endif

#define LOCTEXT_NAMESPACE "MediaStreamFileSchemeHandler"

const FLazyName FMediaStreamFileSchemeHandler::Scheme = TEXT("File");

FMediaStreamSource FMediaStreamFileSchemeHandler::CreateSource(UObject* InOuter, const FString& InPath)
{
	FMediaStreamSource Source;

	if (!IsValid(InOuter))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream in FMediaStreamFileSchemeHandler::CreateSource");
		return Source;
	}

	Source.Scheme = Scheme;

	UMediaSource* MediaSource = CreateMediaSource(InOuter, InPath);

	if (!MediaSource)
	{
		return Source;
	}

	Source.Path = InPath;
	Source.Object = MediaSource;

	return Source;
}

UMediaPlayer* FMediaStreamFileSchemeHandler::CreateOrUpdatePlayer(const FMediaStreamSchemeHandlerCreatePlayerParams& InParams)
{
	if (!IsValid(InParams.MediaStream))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream in FMediaStreamFileSchemeHandler::CreateOrUpdatePlayer");
		return nullptr;
	}

	UObject* MediaObject = InParams.MediaStream->GetSource().Object;

	if (!IsValid(MediaObject))
	{
		MediaObject = CreateMediaSource(InParams.MediaStream, InParams.MediaStream->GetSource().Path);

		if (!IsValid(MediaObject))
		{
			return nullptr;
		}
	}

	UMediaSource* MediaSource = Cast<UMediaSource>(MediaObject);

	if (!IsValid(MediaSource))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Path in FMediaStreamFileSchemeHandler::CreateOrUpdatePlayer [%ls]", *MediaObject->GetClass()->GetName());
		return nullptr;
	}

	return FMediaStreamObjectHandlerManager::Get().CreateOrUpdatePlayer(InParams << MediaSource);
}

FString FMediaStreamFileSchemeHandler::ResolveFilePath(const FString& InPath)
{
	FString FilePath = InPath;

	if (IFileManager::Get().FileExists(*FilePath))
	{
		return FilePath;
	}

	const FString ProjectPath = FPaths::GetProjectFilePath();
	FilePath = FPaths::Combine(ProjectPath, InPath);

	if (IFileManager::Get().FileExists(*FilePath))
	{
		return FilePath;
	}

	const FString EnginePath = FPaths::EngineDir();
	FilePath = FPaths::Combine(EnginePath, InPath);

	if (IFileManager::Get().FileExists(*FilePath))
	{
		return FilePath;
	}

	// Shouldn't generate an error since it's default behavior and common code path. Otherwise it breaks tests.
	UE_LOGF(LogMediaStream, Display, "Invalid File Path in FMediaStreamFileSchemeHandler::ResolveFilePath [%ls]", *InPath);

	return FString();
}

UMediaSource* FMediaStreamFileSchemeHandler::CreateMediaSource(UObject* InOuter, const FString& InPath)
{
	const FString FilePath = ResolveFilePath(InPath);

	if (FilePath.IsEmpty())
	{
		return nullptr;
	}

	UMediaSource* MediaSource = UMediaSource::SpawnMediaSourceForString(FilePath, InOuter);

	if (!MediaSource)
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Path in FMediaStreamFileSchemeHandler::CreateMediaSource [%ls]", *FilePath);
		return nullptr;
	}

	return MediaSource;
}

#if WITH_EDITOR
void FMediaStreamFileSchemeHandler::CreatePropertyCustomization(UMediaStream* InMediaStream, IMediaStreamSchemeHandler::FCustomWidgets& InOutCustomWidgets)
{
	AddFileSelector(InMediaStream, InOutCustomWidgets);
}

void FMediaStreamFileSchemeHandler::AddFileSelector(UMediaStream* InMediaStream, IMediaStreamSchemeHandler::FCustomWidgets& InOutCustomWidgets)
{
	if (!IsValid(InMediaStream))
	{
		return;
	}

	TWeakObjectPtr<UMediaStream> MediaStreamWeak = InMediaStream;

	InOutCustomWidgets.CustomRows.Add({
		LOCTEXT("FilePath", "File Path"),
		SNew(SFilePathPicker)
		.BrowseButtonImage(FAppStyle::GetBrush("PropertyWindow.Button_Ellipsis"))
		.BrowseButtonStyle(FAppStyle::Get(), "HoverHintOnly")
		.BrowseButtonToolTip(LOCTEXT("FileButtonToolTipText", "Choose a file from this computer"))
		.BrowseTitle(LOCTEXT("PropertyEditorTitle", "File picker..."))
		.FilePath_Static(&FMediaStreamSchemeHandlerLibrary::GetPath, MediaStreamWeak)
		.FileTypeFilter(TEXT("All files (*.*)|*.*"))
		.OnPathPicked(this, &FMediaStreamFileSchemeHandler::OnFilePicked, MediaStreamWeak),
		/* Enabled */ true,
		TAttribute<EVisibility>::CreateSP(this, &FMediaStreamFileSchemeHandler::GetFileSelectorVisibility, MediaStreamWeak),
		FMediaStreamSource::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FMediaStreamSource, Path))
	});
}

EVisibility FMediaStreamFileSchemeHandler::GetFileSelectorVisibility(TWeakObjectPtr<UMediaStream> InMediaStreamWeak) const
{
	if (FMediaStreamSchemeHandlerLibrary::GetScheme(InMediaStreamWeak) == Scheme)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

void FMediaStreamFileSchemeHandler::OnFilePicked(const FString& InFilePath, TWeakObjectPtr<UMediaStream> InMediaStreamWeak)
{
	FMediaStreamSchemeHandlerLibrary::SetSource(InMediaStreamWeak, Scheme, InFilePath);
}

#endif

#undef LOCTEXT_NAMESPACE
