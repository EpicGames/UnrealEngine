// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaStreamSchemeHandler.h"

class UMediaSource;
struct EVisibility;

class FMediaStreamFileSchemeHandler : public IMediaStreamSchemeHandler
{
public:
	static const FLazyName Scheme;

	//~ Begin IMediaStreamSchemeHandler
	virtual FMediaStreamSource CreateSource(UObject* InOuter, const FString& InPath) override;
	virtual UMediaPlayer* CreateOrUpdatePlayer(const FMediaStreamSchemeHandlerCreatePlayerParams& InParams) override;

#if WITH_EDITOR
	virtual void CreatePropertyCustomization(UMediaStream* InMediaStream, IMediaStreamSchemeHandler::FCustomWidgets& InOutCustomWidgets) override;
#endif
	//~ End IMediaStreamSchemeHandler

protected:
	FString ResolveFilePath(const FString& InPath);

	UMediaSource* CreateMediaSource(UObject* InOuter, const FString& InPath);

#if WITH_EDITOR
	void AddFileSelector(UMediaStream* InMediaStream, IMediaStreamSchemeHandler::FCustomWidgets& InOutCustomWidgets);

	EVisibility GetFileSelectorVisibility(TWeakObjectPtr<UMediaStream> InMediaStreamWeak) const;

	void OnFilePicked(const FString& InFilePath, TWeakObjectPtr<UMediaStream> InMediaStreamWeak);
#endif
};
