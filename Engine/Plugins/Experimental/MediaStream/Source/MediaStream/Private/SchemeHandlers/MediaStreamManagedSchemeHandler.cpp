// Copyright Epic Games, Inc. All Rights Reserved.

#include "SchemeHandlers/MediaStreamManagedSchemeHandler.h"

#include "MediaStream.h"
#include "MediaStreamModule.h"
#include "MediaStreamSource.h"

const FLazyName FMediaStreamManagedSchemeHandler::Scheme = TEXT("Managed");

FMediaStreamSource FMediaStreamManagedSchemeHandler::CreateSource(UObject* InOuter, const FString& InPath)
{
	FMediaStreamSource Source;

	if (!IsValid(InOuter))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream in FMediaStreamManagedSchemeHandler::CreateSource");
		return Source;
	}

	Source.Scheme = Scheme;

	// TODO: Check valid media source manager stream
	if (false)
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Managed Stream in FMediaStreamManagedSchemeHandler::CreateSource [%ls]", *InPath);
		return Source;
	}

	Source.Path = InPath;

	return Source;
}

UMediaPlayer* FMediaStreamManagedSchemeHandler::CreateOrUpdatePlayer(const FMediaStreamSchemeHandlerCreatePlayerParams& InParams)
{
	// TODO: Get player from media source manager
	return nullptr;
}

#if WITH_EDITOR
void FMediaStreamManagedSchemeHandler::CreatePropertyCustomization(UMediaStream* InMediaStream, IMediaStreamSchemeHandler::FCustomWidgets& InOutCustomWidgets)
{
	// TODO Implement proper customisation
}
#endif
