// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaStreamSchemeHandler.h"

class FMediaStreamManagedSchemeHandler : public IMediaStreamSchemeHandler
{
public:
	static const FLazyName Scheme;

	//~ Begin IMediaStreamSchemeHandler
	virtual FMediaStreamSource CreateSource(UObject* InOuter, const FString& InPath) override;
	virtual UMediaPlayer* CreateOrUpdatePlayer(const FMediaStreamSchemeHandlerCreatePlayerParams& InParams) override;

#if WITH_EDITOR
	virtual void CreatePropertyCustomization(UMediaStream* InMediaStream, IMediaStreamSchemeHandler::FCustomWidgets& InOutCustomWidgets) override;
#endif
	//~ End IMediaStreamSchemeHandler
};
