// Copyright Epic Games, Inc. All Rights Reserved.

#include "SchemeHandlers/MediaStreamSubobjectSchemeHandler.h"

#include "IMediaStreamObjectHandler.h"
#include "MediaStream.h"
#include "MediaStreamModule.h"
#include "MediaStreamObjectHandlerManager.h"
#include "MediaStreamSource.h"

const FLazyName FMediaStreamSubobjectSchemeHandler::Scheme = TEXT("Subobject");

FMediaStreamSource FMediaStreamSubobjectSchemeHandler::CreateSource(UObject* InOuter, const FString& InPath)
{
	FMediaStreamSource Source;

	if (!IsValid(InOuter))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream in FMediaStreamSubobjectSchemeHandler::CreateSource");
		return Source;
	}

	Source.Scheme = Scheme;

	UObject* Object = ResolveSubobjectPath(InOuter, InPath);

	if (!Object)
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Subobject Path in FMediaStreamSubobjectSchemeHandler::CreateSource [%ls]", *InPath);
		return Source;
	}

	Source.Path = InPath;

	return Source;
}

UMediaPlayer* FMediaStreamSubobjectSchemeHandler::CreateOrUpdatePlayer(const FMediaStreamSchemeHandlerCreatePlayerParams& InParams)
{
	if (!IsValid(InParams.MediaStream))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Media Stream in FMediaStreamSubobjectSchemeHandler::CreateOrUpdatePlayer");
		return nullptr;
	}

	UObject* Subobject = ResolveSubobjectPath(InParams.MediaStream, InParams.MediaStream->GetSource().Path);

	if (!IsValid(Subobject))
	{
		UE_LOGF(LogMediaStream, Error, "Invalid Subobject in FMediaStreamSubobjectSchemeHandler::CreateOrUpdatePlayer");
		return nullptr;
	}

	return FMediaStreamObjectHandlerManager::Get().CreateOrUpdatePlayer(InParams << Subobject);
}

UObject* FMediaStreamSubobjectSchemeHandler::ResolveSubobjectPath(UObject* InOuter, const FString& InPath)
{
	const FSoftObjectPath SoftPath = InOuter->GetPathName() + TEXT(".") + InPath;
	return SoftPath.ResolveObject();
}

#if WITH_EDITOR
void FMediaStreamSubobjectSchemeHandler::CreatePropertyCustomization(UMediaStream* InMediaStream, IMediaStreamSchemeHandler::FCustomWidgets& InOutCustomWidgets)
{
	// TODO Implement proper customisation
}
#endif
