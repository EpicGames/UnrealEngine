// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#include "IMediaStreamObjectHandler.generated.h"

class UMediaPlayer;
class UMediaStream;
class UObject;

USTRUCT(BlueprintType)
struct FMediaStreamObjectHandlerCreatePlayerParams
{
	GENERATED_BODY()

	/** The container for the player. */
	UPROPERTY(BlueprintReadWrite, Category = "Media Stream")
	TObjectPtr<UMediaStream> MediaStream;

	/** The media source for the player. */
	UPROPERTY(BlueprintReadWrite, Category = "Media Stream")
	TObjectPtr<UObject> Source;

	/**
	 * The current player to update or null.
	 * If a player is provided, it will be re-used to open the source, if it can be.
	 * If no player is provided, a new player will be created (if allowed).
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Media Stream")
	TObjectPtr<UMediaPlayer> CurrentPlayer;

	/**
	 * Whether the new player can open the source or not.
	 * If this is false, it may mean that a new player is not created or
	 * an existing player is not updated.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Media Stream")
	bool bCanOpenSource = true;
};

/**
 * Implement this interface to add a new object handler.
 * If the derived class has a static UClass* Class then it can be added without specifying it.
 */
class IMediaStreamObjectHandler : public TSharedFromThis<IMediaStreamObjectHandler>
{
public:
	virtual ~IMediaStreamObjectHandler() = default;

	/**
	 * Create or update a UMediaPlayer for the provided source.
	 * Note: This usually means loading the media source. @see bCanLoadSource.
	 * @return A player or nullptr.
	 */
	virtual UMediaPlayer* CreateOrUpdatePlayer(const FMediaStreamObjectHandlerCreatePlayerParams& InParams) = 0;
};
