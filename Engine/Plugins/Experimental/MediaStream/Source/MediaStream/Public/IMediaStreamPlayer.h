// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "MediaStreamEnums.h"
#include "MediaStreamPlayerConfig.h"
#include "MediaStreamTextureConfig.h"

#include "IMediaStreamPlayer.generated.h"

class ULevelSequence;
class UMediaPlayer;
class UMediaStream;
class UMediaTexture;
struct FGuid;
struct FMediaStreamSource;

/**
 * Interface for Media Stream Players. Should only be used within a Media Stream object.
 */
UINTERFACE(MinimalAPI, BlueprintType, Category = "Media Stream", meta = (CannotImplementInterfaceInBlueprint))
class UMediaStreamPlayer : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface for Media Stream Players. Should only be used within a Media Stream object.
 */
class IMediaStreamPlayer
{
	GENERATED_BODY()

public:
	UFUNCTION(Category = "Media Stream|Player")
	virtual UMediaStream* GetMediaStream() const = 0;

	/* Returns whether this player's controls can do anything. */
	UFUNCTION(CallInEditor, Category = "Media Stream|Player")
	virtual bool IsReadOnly() const = 0;

	UFUNCTION(Category = "Media Stream|Player")
	virtual void OnCreated() = 0;

	/**
	 * Called by the owning Media Stream when the source changes so that the player can update.
	 */
	virtual void OnSourceChanged(const FMediaStreamSource& InSource) = 0;

	/**
	 * @return Gets the media texture subobject.
	 */
	UFUNCTION(Category = "Media Stream|Texture")
	virtual UMediaTexture* GetMediaTexture() const = 0;

	/**
	 * @return The current texture config.
	 */
	UFUNCTION(Category = "Media Stream|Texture")
	virtual const FMediaStreamTextureConfig& GetTextureConfig() const = 0;

	/**
	 * Update the current texture's (and any newly set texture's) config.
	 * @param InTextureConfig The new config.
	 */
	UFUNCTION(Category = "Media Stream|Texture")
	virtual void SetTextureConfig(const FMediaStreamTextureConfig& InTextureConfig) = 0;

	/**
	 * Applies the current texture config to the current media texture.
	 * This is generally automatic.
	 */
	UFUNCTION(Category = "Media Stream|Texture")
	virtual void ApplyTextureConfig() = 0;

	/**
	 * @returns The active player, if there is one and it is valid.
	 */
	UFUNCTION(Category = "Media Stream|Player")
	virtual UMediaPlayer* GetPlayer() const = 0;

	/**
	 * @return True if there is an active and valid player.
	 */
	UFUNCTION(Category = "Media Stream|Player")
	virtual bool HasValidPlayer() const = 0;

	/**
	 * @return The current player config.
	 */
	UFUNCTION(Category = "Media Stream|Player")
	virtual const FMediaStreamPlayerConfig& GetPlayerConfig() const = 0;

	/**
	 * Update the current player's (and any newly set player's) config.
	 * @param InPlayerConfig The new config.
	 */
	UFUNCTION(Category = "Media Stream|Player")
	virtual void SetPlayerConfig(const FMediaStreamPlayerConfig& InPlayerConfig) = 0;

	/**
	 * Applies the current player config to the current media player.
	 * This is generally automatic.
	 */
	UFUNCTION(Category = "Media Stream|Player")
	virtual void ApplyPlayerConfig() = 0;

	/**
	 * Setter for events.
	 * @param InIndex The index to start playing.
	 * @return True on success. False on failure or invalid player.
	 */
	UFUNCTION(Category = "Media Stream|Player")
	virtual bool SetPlaylistIndex(int32 InIndex) = 0;

	/**
	 * @return The time offset in the media that has been requested.
	 */
	UFUNCTION(Category = "Media Stream|Player")
	virtual float GetRequestedSeekTime() const = 0;

	/**
	 * Sets the time in the currently playing player, if possible.
	 * @param InTime The time offset in the media to request.
	 * @return True on success. False on failure or invalid player.
	 */
	UFUNCTION(Category = "Media Stream|Player")
	virtual bool SetRequestedSeekTime(float InTime) = 0;

	/**
	 * @return The frame offset in the media that has been requested.
	 */
	UFUNCTION(Category = "Media Stream|Player")
	virtual int32 GetRequestedSeekFrame() const = 0;

	/**
	 * Sets the frame in the currently playing player, if possible.
	 * @param InFrame The frame offset in the media to request.
	 * @return True on success. False on failure or invalid player.
	 */
	UFUNCTION(Category = "Media Stream|Player")
	virtual bool SetRequestedSeekFrame(int32 InFrame) = 0;

	/**
	 * @return Gets the media player playback state.
	 */
	UFUNCTION(Category = "Media Stream|Player")
	virtual EMediaStreamPlaybackState GetPlaybackState() const = 0;

	/**
	 * Sets the media player playback state.
	 * @param InState The state to change to.
	 * @return True on success. False on failure or invalid player.
	 */
	UFUNCTION(Category = "Media Stream|Player")
	virtual bool SetPlaybackState(EMediaStreamPlaybackState InState) = 0;

	/**
	 * @return Gets the active playlist index. May not match the actual player. -1 on error.
	 */
	UFUNCTION(Category = "Media Stream|Player")
	virtual int32 GetPlaylistIndex() const = 0;

	/**
	 * @return The size of the current playlist. -1 on error.
	 */
	UFUNCTION(Category = "Media Stream|Player")
	virtual int32 GetPlaylistNum() const = 0;

	/**
	 * @return For proxy streams, returns the stream this is getting data from.
	 */
	UFUNCTION(Category = "Media Stream|Player")
	virtual UMediaStream* GetSourceStream() const = 0;

	/**
	 * Opens the source if it is not already opened.
	 * @return True if successful.
	 */
	UFUNCTION(CallInEditor, Category = "Media Stream|Player")
	virtual bool OpenSource() = 0;

	/**
	 * Continues play of the current media.
	 * @return True if successful.
	 */
	UFUNCTION(CallInEditor, Category = "Media Stream|Player")
	virtual bool Play() = 0;

	/**
	 * Pauses playback of the current media.
	 * @return True if successful.
	 */
	UFUNCTION(CallInEditor, Category = "Media Stream|Player")
	virtual bool Pause() = 0;

	/**
	 * Seeks to the start of the current media.
	 * @return True if successful.
	 */
	UFUNCTION(CallInEditor, Category = "Media Stream|Player")
	virtual bool Rewind() = 0;

	/**
	 * Seeks to the end of the current media.
	 * @return True if successful.
	 */
	UFUNCTION(CallInEditor, Category = "Media Stream|Player")
	virtual bool FastForward() = 0;

	/**
	 * Goes to the previous item in the playlist. Only possible with playlist sources.
	 * @return True if successful.
	 */
	UFUNCTION(CallInEditor, Category = "Media Stream|Player")
	virtual bool Previous() = 0;

	/**
	 * Goes to the next item in the playlist. Only possible with playlist sources.
	 * @return True if successful.
	 */
	UFUNCTION(CallInEditor, Category = "Media Stream|Player")
	virtual bool Next() = 0;

	/**
	 * Closes the current media player.
	 * @return True if successful.
	 */
	UFUNCTION(CallInEditor, Category = "Media Stream|Player")
	virtual bool Close() = 0;

	/**
	 * Called by the Media Stream when it is done with this Media Stream Player.
	 */
	virtual void Deinitialize() = 0;
};
