// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#include "Containers/ContainersFwd.h"

#if WITH_EDITOR
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#endif

#include "IMediaStreamSchemeHandler.generated.h"

class IPropertyHandle;
class SWidget;
class UMediaPlayer;
class UMediaStream;
struct FMediaStreamSource;
struct FMediaStreamObjectHandlerCreatePlayerParams;

USTRUCT(BlueprintType)
struct FMediaStreamSchemeHandlerCreatePlayerParams
{
	GENERATED_BODY()

	/** The container for the player. */
	UPROPERTY(BlueprintReadWrite, Category = "Media Stream")
	TObjectPtr<UMediaStream> MediaStream;

	/**
	 * The current player to update or null.
	 * If a player is provided, it will be re-used to open the source, if it can be.
	 * If no player is provided, a new player will be created (if allowed).
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Media Stream")
	TObjectPtr<UMediaPlayer> CurrentPlayer;

	/**
	 * Whether the new player can open the source or not.
	 * If this is false, it may mean that a new player is not created or
	 * an existing player is not updated.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Media Stream")
	bool bCanOpenSource = true;

	FMediaStreamObjectHandlerCreatePlayerParams operator<<(UObject* InMediaSource) const;
};

/**
 * Implement this interface to add a new scheme handler.
 * If the derived class has a static TCHAR Scheme[] then it can be added without specifying it.
 */
class IMediaStreamSchemeHandler : public TSharedFromThis<IMediaStreamSchemeHandler>
{
public:
	virtual ~IMediaStreamSchemeHandler() = default;

	/**
	 * Passes a path to produce a stream source.
	 * @param InOuter The container of the source.
	 * @param InPath The path representing the source.
	 * @return A valid stream source object or none if it was not a valid path.
	 */
	virtual FMediaStreamSource CreateSource(UObject* InOuter, const FString& InPath) = 0;

	/**
	 * Create or update a UMediaPlayer for the provided source.
	 * Note: This usually means loading the media source. @see bCanLoadSource.
	 * @return A player or nullptr.
	 */
	virtual UMediaPlayer* CreateOrUpdatePlayer(const FMediaStreamSchemeHandlerCreatePlayerParams& InParams) = 0;

#if WITH_EDITOR
	struct FWidgetRow
	{
		FText Name;
		TSharedRef<SWidget> Widget;
		TAttribute<bool> Enabled = true;
		TAttribute<EVisibility> Visibility = EVisibility::Visible;
		FProperty* SourceProperty = nullptr;
	};

	struct FCustomWidgets
	{
		TArray<FWidgetRow> CustomRows;
	};

	virtual void CreatePropertyCustomization(UMediaStream* InMediaStream, IMediaStreamSchemeHandler::FCustomWidgets& InOutCustomWidgets) = 0;
#endif
};

#if WITH_EDITOR
class FMediaStreamSchemeHandlerLibrary
{
public:
	MEDIASTREAM_API static FMediaStreamSource* GetStreamSourcePtr(TWeakPtr<IPropertyHandle> InPropertyHandleWeak);

	MEDIASTREAM_API static FName GetScheme(TWeakPtr<IPropertyHandle> InPropertyHandleWeak);
	MEDIASTREAM_API static FName GetScheme(UMediaStream* InMediaStream);
	MEDIASTREAM_API static FName GetScheme(TWeakObjectPtr<UMediaStream> InMediaStreamWeak);

	MEDIASTREAM_API static FString GetPath(TWeakPtr<IPropertyHandle> InPropertyHandleWeak);
	MEDIASTREAM_API static FString GetPath(UMediaStream* InMediaStream);
	MEDIASTREAM_API static FString GetPath(TWeakObjectPtr<UMediaStream> InMediaStreamWeak);

	MEDIASTREAM_API static void SetSource(TSharedRef<IPropertyHandle> InPropertyHandle, const FName& InScheme, const FString& InPath);
	MEDIASTREAM_API static void SetSource(UMediaStream* InMediaStream, const FName& InScheme, const FString& InPath);
	MEDIASTREAM_API static void SetSource(TWeakObjectPtr<UMediaStream> InMediaStreamWeak, const FName& InScheme, const FString& InPath);
};
#endif
