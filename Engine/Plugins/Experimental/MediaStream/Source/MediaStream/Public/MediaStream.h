// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "MediaPlayerProxyInterface.h"
#include "UObject/Object.h"

#include "MediaStreamSource.h"
#include "UObject/ScriptInterface.h"

#include "MediaStream.generated.h"

class IMediaStreamPlayer;

/**
 * Media Stream. Provides an agnostic interface between controllers and players.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, ClassGroup = "Media Stream", DefaultToInstanced, 
	PrioritizeCategories = (MediaStream, MediaControls, MediaSource, MediaDetails, MediaTexture, MediaCache, MediaPlayer))
class UMediaStream : public UObject, public IMediaPlayerProxyInterface
{
	GENERATED_BODY()

public:
	MEDIASTREAM_API static FName GetSourcePropertyName();
	MEDIASTREAM_API static FName GetPlayerPropertyName();

	virtual ~UMediaStream() override = default;

	/**
	 * @return True if the media source scheme is not empty. Does not guarantee the source is correct, only valid.
	 */
	UFUNCTION(Category = "Media Stream")
	MEDIASTREAM_API bool HasValidSource() const;

	/**
	 * @return The current Media Stream Source. Does not guaranteed that it is valid.
	 */
	UFUNCTION(Category = "Media Stream")
	MEDIASTREAM_API const FMediaStreamSource& GetSource() const;

	/**
	 * @return The source at the end of the chain of proxy players, or the local one if it is local. If
	 * the chain is interrupted, the last link in the chain's source will be returned.
	 */
	UFUNCTION(Category = "Media Stream")
	MEDIASTREAM_API const FMediaStreamSource& ResolveSource() const;

	/**
	 * Sets a new Media Stream Source and inits the Player. It should be ready to use if this returns true.
	 * @param InSource The new source.
	 * @return True if the source was successfully applied.
	 */
	UFUNCTION(Category = "Media Stream")
	MEDIASTREAM_API bool SetSource(const FMediaStreamSource& InSource);

	/**
	 * @return Gets the active Media Stream Player.
	 */
	UFUNCTION(Category = "Media Stream")
	MEDIASTREAM_API TScriptInterface<IMediaStreamPlayer> GetPlayer() const;

	/**
	 * Creates a player if it doesn't exist.
	 * @param  bInForceRecreatePlayer Will remove and recreate the player.
	 * @return True if a valid player exists.
	 */
	UFUNCTION(Category = "Media Stream")
	MEDIASTREAM_API bool EnsurePlayer(bool bInForceRecreatePlayer = false);

	/**
	 * Called when the Media Stream Source changes.
	 */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnSourceChanged, UMediaStream*, InMediaStream);
	MEDIASTREAM_API FOnSourceChanged& GetOnSourceChanged();

	/**
	 * Called when the Media Stream Player or its Settings change.
	 */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPlayerChanged, UMediaStream*, InMediaStream);
	MEDIASTREAM_API FOnPlayerChanged& GetOnPlayerChanged();

	/**
	 * Fires after the player has been created (so GetPlayer()->GetMediaTexture() is valid) but
	 * before its IMediaStreamPlayer::OnSourceChanged runs. Skipped when the new source is invalid.
	 * Handlers must not re-enter SetSource synchronously.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPreSourceChange, UMediaStream* /*InMediaStream*/);
	MEDIASTREAM_API FOnPreSourceChange& GetOnPreSourceChange();

	/**
	 * Stops the media stream and unloads any resources.
	 */
	MEDIASTREAM_API void Close();

	//~ Begin UObject
#if WITH_EDITOR
	virtual void PreEditUndo() override;
	virtual void PostEditUndo() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	virtual void PostDuplicate(bool bInDuplicateForPIE) override;
	virtual void PostEditImport() override;
	virtual void PostNetReceive() override;
	virtual void BeginDestroy() override;
	//~ End UObject

	//~ Begin IMediaPlayerProxyInterface
	virtual float GetProxyRate() const override;
	virtual bool SetProxyRate(float Rate) override;
	virtual bool IsExternalControlAllowed() override;
	virtual const FMediaSourceCacheSettings& GetCacheSettings() const override;
	virtual UMediaSource* ProxyGetMediaSourceFromIndex(int32 Index) const override;
	virtual UMediaTexture* ProxyGetMediaTexture(int32 LayerIndex, int32 TextureIndex) override;
	virtual void ProxyReleaseMediaTexture(int32 LayerIndex, int32 TextureIndex) override;
	virtual bool ProxySetAspectRatio(UMediaPlayer* InMediaPlayer) override;
	virtual void ProxySetTextureBlend(int32 LayerIndex, int32 TextureIndex, float Blend) override;
	//~ End IMediaPlayerProxyInterface

protected:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media Source")
	FMediaStreamSource Source;

	/**
	 * The player instance is automatically created based on the type of source.
	 * Where possible, the player is re-used when opening new media.
	 */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Instanced, DisplayName = "Player", Category = "Media Stream")
	TObjectPtr<UObject> PlayerObject = nullptr;

	UPROPERTY(BlueprintAssignable)
	FOnSourceChanged OnSourceChanged;

	UPROPERTY(BlueprintAssignable)
	FOnPlayerChanged OnPlayerChanged;

	/** Fired by ApplySource between player creation and IMediaStreamPlayer::OnSourceChanged. See GetOnPreSourceChange. */
	FOnPreSourceChange OnPreSourceChange;

	/** True while ApplySource is in progress; blocks recursive SetSource calls from delegate handlers. */
	bool bIsApplyingSource = false;

	FMediaStreamSource Source_PreUndo;

	/**
	 * Applies the current source settings to this Media Stream, potentially changing the Media Stream Player.
	 * @return True if it was successful.
	 */
	bool ApplySource();
};
