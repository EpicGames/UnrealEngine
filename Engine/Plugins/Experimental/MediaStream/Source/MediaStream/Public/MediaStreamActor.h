// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/StaticMeshComponent.h"

#include "MediaStreamActor.generated.h"

class UMediaStreamComponent;

UCLASS(BlueprintType, MinimalAPI, ComponentWrapperClass, NotPlaceable,
	PrioritizeCategories = (MediaStream, MediaControls, MediaSource, MediaDetails, MediaTexture, MediaCache, MediaPlayer))
class AMediaStreamActor : public AActor
{
	GENERATED_BODY()

public:
	AMediaStreamActor();

protected:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Media Stream")
	TObjectPtr<UMediaStreamComponent> MediaStreamComponent;
};
