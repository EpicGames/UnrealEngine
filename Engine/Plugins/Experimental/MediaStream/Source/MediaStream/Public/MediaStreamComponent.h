// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/StaticMeshComponent.h"

#include "MediaStreamComponent.generated.h"

class IMediaStreamPlayer;
class UMediaStream;

UCLASS(BlueprintType, MinimalAPI,
	PrioritizeCategories = (MediaStream, MediaControls, MediaSource, MediaDetails, MediaTexture, MediaCache, MediaPlayer))
class UMediaStreamComponent : public UStaticMeshComponent
{
	GENERATED_BODY()

public:
	UMediaStreamComponent();

	//~ Begin UObject
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
	virtual void PostNetReceive() override;
	//~ End UObject

protected:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Media Stream")
	TObjectPtr<UMediaStream> MediaStream;

	UPROPERTY(BlueprintReadOnly, Transient, DuplicateTransient, TextExportTransient, Category = "Media Stream")
	TObjectPtr<UMaterialInstanceDynamic> MaterialInstance;

	UFUNCTION()
	void OnSourceChanged(UMediaStream* InMediaStream);

	void InitPlayer();
};
