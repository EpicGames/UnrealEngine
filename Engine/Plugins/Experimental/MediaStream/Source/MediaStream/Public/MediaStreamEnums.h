// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#include "MediaStreamEnums.generated.h"

UENUM(BlueprintType)
enum class EMediaStreamPlaybackState : uint8
{
	Play,
	Pause
};

UENUM(BlueprintType)
enum class EMediaStreamPlaybackDirection : uint8
{
	Forward,
	Backward
};

UENUM(BlueprintType)
enum class EMediaStreamPlaybackSeek : uint8
{
	Previous,
	Start,
	End,
	Next
};
