// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Logging/LogMacros.h"

class IMediaStreamPlayer;
enum class EMediaStreamPlaybackState : uint8;
struct FCanLoadMap;

DECLARE_LOG_CATEGORY_EXTERN(LogMediaStream, Log, All);

/**
 * Media Stream - Content/type agnostic chainable media proxy with media player integration.
 */
class FMediaStreamModule : public IModuleInterface
{
public:
	MEDIASTREAM_API static FMediaStreamModule& Get();

	MEDIASTREAM_API bool CanOpenSourceOnLoad();

	MEDIASTREAM_API bool CanAutoplay() const;

	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

#if WITH_EDITOR
protected:
	bool bIsMapLoading;

	void OnMapLoad(const FString& InFilename, FCanLoadMap& OutCanLoadMap);

	void OnMapOpened(const FString& InFilename, bool bInTemplate);
#endif
};
