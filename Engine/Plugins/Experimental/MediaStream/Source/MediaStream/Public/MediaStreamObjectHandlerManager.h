// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class IMediaStreamObjectHandler;
class UMediaPlayer;
class UMediaStream;
struct FMediaStreamObjectHandlerCreatePlayerParams;
struct FMediaStreamSource;

/**
 * Handles the registration and operation of object handlers.
 * Will fall back to using the Media Assets module if no handler is found.
 */
class FMediaStreamObjectHandlerManager
{
	friend class UMediaStreamSourceBlueprintLibrary;

public:
	/**
	 * @return Gets the instance of this manager.
	 */
	MEDIASTREAM_API static FMediaStreamObjectHandlerManager& Get();

	/**
	 * Checks whether the given class has a class handler. Checks each super class too.
	 * @param InClass The class to check.
	 * @return True if any handler can handle the class.
	 */
	MEDIASTREAM_API bool CanHandleObject(const UClass* InClass) const;

	/**
	 * Templated version of CanHandleObject.
	 * @tparam InClassName The class to check.
	 * @return True if any handler can handle the class.
	 */
	template<typename InClassName
		UE_REQUIRES(TModels_V<CStaticClassProvider, InClassName>)>
	bool CanHandleObject() const
	{
		return CanHandleObject(InClassName::StaticClass());
	}

	/**
	 * Checks whether the given object's class has a class handler. Checks each super class too.
	 * @param InObject The object whose class is to be checked.
	 * @return True if any handler can handle the object's class.
	 */
	MEDIASTREAM_API bool CanHandleObject(const UObject* InObject) const;

	/**
	 * Checks whether a class has a handler registered.
	 * @param InClass The class to check. Must be an exact match.
	 * @return True if the handler is registered.
	 */
	MEDIASTREAM_API bool HasObjectHandler(const UClass* InClass) const;

	/**
	 * Checks whether an object's class has a handler registered.
	 * @param InObject The object whose class to check. Must be an exact match.
	 * @return True if the handler is registered.
	 */
	MEDIASTREAM_API bool HasObjectHandler(const UObject* InObject) const;

	/**
	 * Templated version of HasObjectHandler.
	 * @tparam InClassName The class to check. Must be an exact match.
	 * @return True if the handler is registered.
	 */
	template<typename InClassName
		UE_REQUIRES(TModels_V<CStaticClassProvider, InClassName>)>
	bool HasObjectHandler() const
	{
		return HasObjectHandler(InClassName::StaticClass());
	}

	/**
	 * Does an hierarchical search for a handler that can handle the given class.
	 * @param InClass A validity checked UObject class.
	 * @return The handler for the class, if found.
	 */
	MEDIASTREAM_API TSharedPtr<IMediaStreamObjectHandler> GetHandler(const UClass* InClass) const;

	/**
	 * Registers a Object handler that creates a media player for the given object class. Holds a strong reference.
	 * Does not replace an already registered handler.
	 * @param InClass The class of the object handled.
	 * @param InHandler The handle which creates the media player.
	 * @return True if the handler was registered.
	 */
	MEDIASTREAM_API bool RegisterObjectHandler(const UClass* InClass, const TSharedRef<IMediaStreamObjectHandler>& InHandler);

	/**
	 * Templated version of RegisterObjectHandler.
	 * @tparam InClassName The class of the object handled.
	 * @param InHandler The handle which creates the media player.
	 * @return True if the handler was registered.
	 */
	template<typename InClassName
		UE_REQUIRES(TModels_V<CStaticClassProvider, InClassName>)>
	bool RegisterObjectHandler(const TSharedRef<IMediaStreamObjectHandler>& InHandler)
	{
		return RegisterObjectHandler(InClassName::StaticClass(), InHandler);
	}

	/**
	 * Templated version of RegisterObjectHandler.
	 * @tparam InHandlerClass Class of the handle which creates the media player.
	 * @tparam InArgsType The type of handle class constructor parameters (automatic).
	 * @param InClass The class of the object handled.
	 * @param InArgs The handle class constructor parameters.
	 * @return True if the handler was registered.
	 */
	template<typename InHandlerClass, typename... InArgsType
		UE_REQUIRES(TIsDerivedFrom<InHandlerClass, IMediaStreamObjectHandler>::Value)>
	bool RegisterObjectHandler(const UClass* InClass, InArgsType&&... InArgs)
	{
		return RegisterObjectHandler(InClass, MakeShared<InHandlerClass>(Forward<InArgsType>(InArgs)...));
	}

	/**
	 * Templated version of RegisterObjectHandler.
	 * @tparam InClassName The class of the object handled.
	 * @tparam InHandlerClass Class of the handle which creates the media player.
	 * @tparam InArgsType The type of handle class constructor parameters (automatic).
	 * @param InArgs The handle class constructor parameters.
	 * @return True if the handler was registered.
	 */
	template<typename InClassName, typename InHandlerClass, typename... InArgsType
		UE_REQUIRES(TAnd<TModels<CStaticClassProvider, InClassName>,
			TIsDerivedFrom<InHandlerClass, IMediaStreamObjectHandler>>::Value)>
		bool RegisterObjectHandler(InArgsType&&... InArgs)
	{
		return RegisterObjectHandler(InClassName::StaticClass(), MakeShared<InHandlerClass>(Forward<InArgsType>(InArgs)...));
	}

	/**
	 * Templated version of RegisterObjectHandler with auto Class value.
	 * @tparam InHandlerClass Class of the handle which creates the media player.
	 * @tparam InArgsType The type of handle class constructor parameters (automatic).
	 * @param InArgs The handle class constructor parameters.
	 * @return True if the handler was registered.
	 */
	template<typename InHandlerClass, typename... InArgsType
		UE_REQUIRES(TIsDerivedFrom<InHandlerClass, IMediaStreamObjectHandler>::Value)>
	bool RegisterObjectHandler(InArgsType&&... InArgs)
	{
		return RegisterObjectHandler(InHandlerClass::GetClass(), MakeShared<InHandlerClass>(Forward<InArgsType>(InArgs)...));
	}

	/**
	 * Unregisters a Object handler.
	 * @param InClass The class to unregister.
	 * @return The removed Object handler, if found.
	 */
	MEDIASTREAM_API TSharedPtr<IMediaStreamObjectHandler> UnregisterObjectHandler(const UClass* InClass);

	/**
	 * Templated version of UnregisterObjectHandler.
	 * @tparam InClassName The class to unregister.
	 * @return The removed Object handler, if found.
	 */
	template<typename InClassName
		UE_REQUIRES(TModels_V<CStaticClassProvider, InClassName>)>
	TSharedPtr<IMediaStreamObjectHandler> UnregisterObjectHandler()
	{
		return RegisterObjectHandler(InClassName::StaticClass());
	}

	/**
	 * Templated version of UnregisterObjectHandler with auto Class value.
	 * @tparam InHandlerClass Class of the handle which creates the media player.
	 * @return The removed Object handler, if found.
	 */
	template<typename InHandlerClass
		UE_REQUIRES(TIsDerivedFrom<InHandlerClass, IMediaStreamObjectHandler>::Value)>
	TSharedPtr<InHandlerClass> UnregisterObjectHandler()
	{
		return StaticCastSharedPtr<InHandlerClass>(UnregisterObjectHandler(InHandlerClass::GetClass()));
	}

	/**
	 * Create or update a UMediaPlayer for the provided source.
	 * Note: This usually means loading the media source. @see bCanLoadSource.
	 * @return A player or nullptr.
	 */
	MEDIASTREAM_API UMediaPlayer* CreateOrUpdatePlayer(const FMediaStreamObjectHandlerCreatePlayerParams& InParams) const;

private:
	TMap<FName, TSharedRef<IMediaStreamObjectHandler>> Handlers;
};
