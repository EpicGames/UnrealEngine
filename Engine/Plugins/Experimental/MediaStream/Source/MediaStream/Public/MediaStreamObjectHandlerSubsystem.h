// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"

#include "MediaStreamObjectHandlerSubsystem.generated.h"

class UMediaPlayer;
class UMediaStream;
struct FMediaStreamObjectHandlerCreatePlayerParams;

/**
 * Subsystem for blueprint-style interaction with the Object Handler Manager.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Media Stream")
class UMediaStreamObjectHandlerSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	/**
	 * @return Gets the instance of this subsystem.
	 */
	UFUNCTION(Category = "Media Stream|Object")
	static MEDIASTREAM_API UMediaStreamObjectHandlerSubsystem* Get();

	/**
	 * Checks whether the given class has a class handler. Checks each super class too.
	 * @param InClass The class to check.
	 * @return True if any handler can handle the class.
	 */
	UFUNCTION(Category = "Media Stream|Object")
	MEDIASTREAM_API bool CanHandleObject(const UClass* InClass) const;

	/**
	 * Checks whether a class has a handler registered.
	 * @param InClass The class to check. Must be an exact match.
	 * @return True if the handler is registered.
	 */
	UFUNCTION(Category = "Media Stream|Object")
	MEDIASTREAM_API bool HasObjectHandler(const UClass* InClass) const;

	/**
	 * Create or update a UMediaPlayer for the provided source.
	 * Note: This usually means loading the media source. @see bCanLoadSource.
	 * @return A player or nullptr.
	 */
	UFUNCTION(Category = "Media Stream|Object")
	MEDIASTREAM_API UMediaPlayer* CreateMediaPlayer(const FMediaStreamObjectHandlerCreatePlayerParams& InParams) const;
};
