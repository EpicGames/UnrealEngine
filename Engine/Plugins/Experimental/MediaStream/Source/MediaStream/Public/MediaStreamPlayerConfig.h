// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaPlayer.h"

#include "MediaStreamPlayerConfig.generated.h"

class UMediaPlayer;

USTRUCT(BlueprintType)
struct FMediaStreamPlayerConfig
{
	GENERATED_BODY()

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Media Stream|Player")
	bool bPlayOnOpen = true;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Media Stream|Player", EditFixedSize, meta = (EditFixedOrder))
	FMediaPlayerTrackOptions TrackOptions;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Media Stream|Player")
	bool bLooping = false;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Media Stream|Player", meta = (UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1))
	float Volume = 1.f;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Media Stream|Player")
	TOptional<FFloatInterval> PlaybackTimeRange;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Media Stream|Player")
	float Rate = 1.f;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Media Stream|Player", meta = (Units = "s"))
	float TimeDelay = 0.f;

	/** Only applicable to play lists. */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Media Stream|Player")
	bool bShuffle = false;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Media Stream|Player", 
		meta = (Units = "s", UIMin = 0.1, ClampMin = 0.01, UIMax = 5, ClampMax = 60))
	float CacheAhead = 0.2f;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Media Stream|Player",
		meta = (Units = "s", UIMin = 0.1, ClampMin = 0.01, UIMax = 5, ClampMax = 60))
	float CacheBehind = 3.0f;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Media Stream|Player",
		meta = (Units = "s", UIMin = 0.1, ClampMin = 0.01, UIMax = 5, ClampMax = 60))
	float CacheBehindGame = 0.1f;

	/**
	 * Enables a per-player local sample cache override instead of the default global cache settings.
	 * Currently maps to the ImgMediaPlayer smart cache, with CacheAhead reused as the look-ahead time.
	 */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Media Stream|Player")
	bool bUseLocalCache = false;

	MEDIASTREAM_API bool operator==(const FMediaStreamPlayerConfig& InOther) const;

	FMediaPlayerOptions CreateOptions(const FTimespan& InStartTime, const TMap<FName, FVariant>& InCustomOptions = {}) const;

	void ApplyConfig(UMediaPlayer& InMediaPlayer) const;

	bool ApplyRate(UMediaPlayer& InMediaPlayer) const;
};
