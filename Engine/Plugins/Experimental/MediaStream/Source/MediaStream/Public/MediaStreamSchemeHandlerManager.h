// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class IMediaStreamSchemeHandler;
class UMediaPlayer;
class UMediaStream;
struct FMediaStreamSchemeHandlerCreatePlayerParams;
struct FMediaStreamSource;

/**
 * Handles the registration and operation of Scheme Handlers.
 */
class FMediaStreamSchemeHandlerManager
{
public:
	/**
	 * @return Gets the instance of this manager.
	 */
	static MEDIASTREAM_API FMediaStreamSchemeHandlerManager& Get();

	/**
	 * Checks whether a scheme has a handler registered.
	 * @param InScheme The scheme to check.
	 * @return True if the handler is registered.
	 */
	MEDIASTREAM_API bool HasSchemeHandler(FName InScheme) const;

	/**
	 * @return Gets the list of registered scheme handlers.
	 */
	MEDIASTREAM_API TArray<FName> GetSchemeHandlerNames() const;

	/**
	 * Registers a Scheme handler. Holds a strong reference.
	 * Does not replace an already registered handler.
	 * @param InScheme The scheme name.
	 * @param InHandler The handler which converts the path to a stream source.
	 * @return True if the handler was registered.
	 */
	MEDIASTREAM_API bool RegisterSchemeHandler(FName InScheme, const TSharedRef<IMediaStreamSchemeHandler>& InHandler);

	/**
	 * Templated version of RegisterSchemeHandler.
	 * @tparam InHandlerClass The handler class to register.
	 * @tparam InArgsType The type of handle class constructor parameters (automatic).
	 * @param InScheme The scheme to register.
	 * @param InArgs The handle class constructor parameters.
	 * @return True if the handler was registered.
	 */
	template<typename InHandlerClass, typename... InArgsType
		UE_REQUIRES(TIsDerivedFrom<InHandlerClass, IMediaStreamSchemeHandler>::Value)>
	bool RegisterSchemeHandler(FName InScheme, InArgsType&&... InArgs)
	{
		return RegisterSchemeHandler(InScheme, MakeShared<InHandlerClass>(Forward<InArgsType>(InArgs)...));
	}

	/**
	 * Templated version of RegisterSchemeHandler with auto scheme value.
	 * @tparam InHandlerClass The handler class to register.
	 * @tparam InArgsType The type of handle class constructor parameters (automatic).
	 * @param InArgs The handle class constructor parameters.
	 * @return True if the handler was registered.
	 */
	template<typename InHandlerClass, typename... InArgsType
		UE_REQUIRES(TIsDerivedFrom<InHandlerClass, IMediaStreamSchemeHandler>::Value)>
	bool RegisterSchemeHandler(InArgsType&&... InArgs)
	{
		return RegisterSchemeHandler(InHandlerClass::Scheme, MakeShared<InHandlerClass>(Forward<InArgsType>(InArgs)...));
	}

	/**
	 * Unregisters a Scheme handler.
	 * @param InScheme The scheme to unregister.
	 * @return The removed Scheme handler, if found.
	 */
	MEDIASTREAM_API TSharedPtr<IMediaStreamSchemeHandler> UnregisterSchemeHandler(FName InScheme);

	/**
	 * Templated version of UnregisterSchemeHandler with auto scheme value.
	 * @tparam InHandlerClass The handler class to register.
	 * @return The removed Scheme handler, if found.
	 */
	template<typename InHandlerClass
		UE_REQUIRES(TIsDerivedFrom<InHandlerClass, IMediaStreamSchemeHandler>::Value)>
	TSharedPtr<IMediaStreamSchemeHandler> UnregisterSchemeHandler()
	{
		return UnregisterSchemeHandler(InHandlerClass::Scheme);
	}

	/**
	 * Finds the handler which is used for the given scheme.
	 * @param InScheme The scheme to check.
	 * @return The found handler or nullptr.
	 */
	MEDIASTREAM_API TSharedPtr<IMediaStreamSchemeHandler> GetHandlerTypeForScheme(FName InScheme) const;

	/**
	 * Create a Media Stream Source from a scheme and path. Must have a registered handler.
	 * @param InOuter The container of the source.
	 * @param InScheme The scheme used to resolve the path.
	 * @param InPath The path to the media source. This should not be Url encoded.
	 * @return A valid stream source object or none if it was not a valid Url.
	 */
	MEDIASTREAM_API FMediaStreamSource CreateSource(UObject* InOuter, FName InScheme, const FString& InPath) const;

	/**
	 * Create or update a UMediaPlayer for the provided source.
	 * Note: This usually means loading the media source. @see bCanLoadSource.
	 * @return A player or nullptr.
	 */
	MEDIASTREAM_API UMediaPlayer* CreateOrUpdatePlayer(const FMediaStreamSchemeHandlerCreatePlayerParams& InParams) const;

private:
	TMap<FName, TSharedRef<IMediaStreamSchemeHandler>> Handlers;
};
