// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"

#include "HAL/Platform.h"
#include "UObject/NameTypes.h"

#include "MediaStreamSchemeHandlerSubsystem.generated.h"

class UMediaPlayer;
class UMediaStream;
struct FMediaStreamSchemeHandlerCreatePlayerParams;
struct FMediaStreamSource;

/**
 * Subsystem for blueprint-style interaction with the Scheme Handler Manager.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Media Stream")
class UMediaStreamSchemeHandlerSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	/**
	 * @return Gets the instance of this subsystem.
	 */
	UFUNCTION(Category = "Media Stream|Scheme")
	static MEDIASTREAM_API UMediaStreamSchemeHandlerSubsystem* Get();

	/**
	 * Checks whether a scheme has a handler registered.
	 * @param InScheme The scheme to check.
	 * @return True if the handler is registered.
	 */
	UFUNCTION(Category = "Media Stream|Scheme")
	MEDIASTREAM_API bool HasSchemeHandler(FName InScheme) const;

	/**
	 * @return Gets the list of registered scheme handlers.
	 */
	UFUNCTION(Category = "Media Stream|Scheme")
	MEDIASTREAM_API TArray<FName> GetSchemeHandlerNames() const;

	/**
	 * Create a Media Stream Source from a scheme and path. Must have a registered handler.
	 * @param InOuter The container of the source.
	 * @param InScheme The scheme used to resolve the path.
	 * @param InPath The path to the media source. This should not be Url encoded.
	 * @return A valid stream source object or none if it was not a valid Url.
	 */
	UFUNCTION(Category = "Media Stream|Scheme")
	FMediaStreamSource CreateSource(UObject* InOuter, FName InScheme, const FString& InPath) const;

	/**
	 * Create or update a UMediaPlayer for the provided source.
	 * Note: This usually means loading the media source. @see bCanLoadSource.
	 * @return A player or nullptr.
	 */
	UFUNCTION(Category = "Media Stream|Scheme")
	MEDIASTREAM_API UMediaPlayer* CreateOrUpdatePlayer(const FMediaStreamSchemeHandlerCreatePlayerParams& InParams) const;
};
