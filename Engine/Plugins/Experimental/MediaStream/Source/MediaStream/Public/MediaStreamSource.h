// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"

#include "MediaStreamSource.generated.h"

class UMediaPlayer;
class UMediaSource;
class UMediaStream;

/**
 * Represents the source of a Media Stream.
 *
 * This uses the idea of a "scheme" and "path" to reference an asset.
 * This nomenclature was chosen so that it would be easy to relate url schemes to
 * this system, such as file://path or http://path.
 */
USTRUCT(BlueprintType)
struct FMediaStreamSource
{
	GENERATED_BODY()

	/**
	 * Used by the handler subsystem to identifer the scheme used.
	 * A value of NAME_None identifies a null or invalid source.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Media Stream|Source",
		meta = (GetOptions = "/Script/MediaStream.MediaStreamSourceBlueprintFunctionLibrary:GetSchemeTypes"))
	FName Scheme = NAME_None;

	/**
	 * Path to the referenced source, such as a file/asset path or a managed source name.
	 * A path must be provided to have a valid scheme.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Media Stream|Source")
	FString Path;

	/**
	 * Source of the Media. Either a UMediaStream, UMediaSource or a UMediaPlaylist
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Media Stream|Source",
		meta = (AllowedClasses="/Script/MediaStream.MediaStream,/Script/MediaAssets.MediaSource,/Script/MediaAssets.MediaPlaylist"))
	TObjectPtr<UObject> Object = nullptr;

	MEDIASTREAM_API bool operator==(const FMediaStreamSource& InOther) const;
};

UCLASS()
class UMediaStreamSourceBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION()
	static TArray<FName> GetSchemeTypes();
};