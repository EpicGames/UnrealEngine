// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "Containers/StringFwd.h"
#include "MediaStreamSource.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"

#include "MediaStreamSourceBlueprintLibrary.generated.h"

/**
 * Deals with creating new Media Stream Sources.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Media Stream")
class UMediaStreamSourceBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Checks the media stream to see if its source type is none or set to something valid.
	 * And that the path is not empty.
	 * @param InSource The source to check.
	 * @return True if the source is valid. Does not guarantee it is correct, only valid.
	 */
	UFUNCTION(Category = "Media Stream|Source")
	static MEDIASTREAM_API bool IsValidMediaSource(const FMediaStreamSource& InSource);

	/**
	 * Checks the Asset pointer to see if it's a valid asset for a Media Stream Source.
	 * @param InAsset The asset to check.
	 * @return True if the asset is valid.
	 */
	UFUNCTION(Category = "Media Stream|Source")
	static MEDIASTREAM_API bool IsAssetValid(const TSoftObjectPtr<UObject>& InAsset);

	/**
	 * Checks the Asset path to see if it's a valid asset for a Media Stream Source.
	 * @param InPath The asset path to check.
	 * @return True if the asset is valid.
	 */
	UFUNCTION(Category = "Media Stream|Source")
	static MEDIASTREAM_API bool IsAssetPathValid(const FString& InPath);

	/**
	 * Checks the Asset soft path to see if it's a valid asset for a Media Stream Source.
	 * @param InPath The asset soft path to check.
	 * @return True if the asset is valid.
	 */
	UFUNCTION(Category = "Media Stream|Source")
	static MEDIASTREAM_API bool IsAssetSoftPathValid(const FSoftObjectPath& InPath);

	/**
	 * Create a Media Stream Source from an Asset. Must have a registered handler.
	 * @param InMediaStream The owning Media Stream
	 * @param InScheme The scheme of the source.
	 * @param InPath The path to the source.
	 * @return A valid stream source object or none if it was not a valid object.
	 */
	UFUNCTION(Category = "Media Stream|Source")
	static MEDIASTREAM_API FMediaStreamSource MakeMediaSourceFromSchemePath(UMediaStream* InMediaStream, FName InScheme, const FString& InPath);

	/**
	 * Create a Media Stream Source from an Asset. Must have a registered handler.
	 * @param InMediaStream The owning Media Stream
	 * @param InObject The playable object.
	 * @return A valid stream source object or none if it was not a valid object.
	 */
	UFUNCTION(Category = "Media Stream|Source")
	static MEDIASTREAM_API FMediaStreamSource MakeMediaSourceFromAsset(UMediaStream* InMediaStream, const TSoftObjectPtr<UObject>& InAsset);

	/**
	 * Create a Media Stream Source from a stream name. Must be set up via the Media Source Manager.
	 * @param InMediaStream The owning Media Stream
	 * @param InStreamName The name of the registered stream.
	 * @return A valid stream source object or none if it was not a valid stream or the stream was not found.
	 */
	UFUNCTION(Category = "Media Stream|Source")
	static MEDIASTREAM_API FMediaStreamSource MakeMediaSourceFromStreamName(UMediaStream* InMediaStream, FName InStreamName);

	/**
	 * Create a Media Stream Source from a file name.
	 * @param InMediaStream The owning Media Stream
	 * @param InFileName The path to the file name. Should be a full path or relative to the project or engine root.
	 * @return A valid stream source object or none if the file was not found.
	 */
	UFUNCTION(Category = "Media Stream|Source")
	static MEDIASTREAM_API FMediaStreamSource MakeMediaSourceFromFile(UMediaStream* InMediaStream, const FString& InFileName);

	/**
	 * Create a Media Stream Source from an object.
	 * @param InMediaStream The owning Media Stream
	 * @param InRootObject Root of the relative path.
	 * @param InObject The playable object. Should share a common ancestor with the Media Stream, but this is not enforced.
	 * @return A valid stream source object or none if the file was not found.
	 */
	UFUNCTION(Category = "Media Stream|Source")
	static MEDIASTREAM_API FMediaStreamSource MakeMediaSourceFromSubobject(UMediaStream* InMediaStream, UObject* InObject);

	/**
	 * Create a Media Stream Source by instantiating the given class.
	 * @param InMediaStream The owning Media Stream
	 * @param InClass The class of a playable object.
	 * @param InOuter The outer of the new object. Defaults to the media stream object.
	 * @return A valid stream source object or none if the file was not found.
	 */
	UFUNCTION(Category = "Media Stream|Source")
	static MEDIASTREAM_API FMediaStreamSource MakeMediaSourceFromSubobjectClass(UMediaStream* InMediaStream, const UClass* InClass);

	/**
	 * Templated version of MakeMediaSourceFromClass.
	 * @tparam InClassName The name of the UClass to create an playable from.
	 * @param InMediaStream The owning Media Stream
	 * @param InOuter The outer of the newly created object
	 * @return A valid stream source object or none if the file was not found.
	 */
	template<typename InClassName
		UE_REQUIRES(TModels_V<CStaticClassProvider, InClassName>)>
	static FMediaStreamSource MakeMediaSourceFromSubobject(UMediaStream* InMediaStream)
	{
		return MakeMediaSourceFromSubobjectClass(InMediaStream, InClassName::StaticClass());
	}
};
