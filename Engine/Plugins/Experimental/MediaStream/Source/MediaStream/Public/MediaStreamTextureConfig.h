// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaTexture.h"

#include "MediaStreamTextureConfig.generated.h"

class UMediaTexture;

USTRUCT(BlueprintType)
struct FMediaStreamTextureConfig
{
	GENERATED_BODY()

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Media Stream|Texture", DisplayName = "Enable Real Time Mips")
	bool bEnableMipGen = false;

	MEDIASTREAM_API bool operator==(const FMediaStreamTextureConfig& InOther) const;

	void ApplyConfig(UMediaTexture& InMediaTexture) const;
};
