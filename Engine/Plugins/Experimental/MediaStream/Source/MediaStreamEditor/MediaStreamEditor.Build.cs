// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MediaStreamEditor : ModuleRules
{
	public MediaStreamEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Engine"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"DesktopWidgets",
				"LevelSequence",
				"LevelSequenceEditor",
				"MediaAssets",
				"MediaCompositing",
				"MediaPlayerEditor",
				"MediaStream",
				"MediaUtils",
				"MovieScene",
				"Projects",
				"PropertyEditor",
				"Sequencer",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"UnrealEd",
			}
		);
	}
}
