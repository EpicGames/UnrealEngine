// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsPanel/MediaStreamCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "MediaStream.h"
#include "MediaStreamEditorSequencerLibrary.h"
#include "MediaStreamEditorStyle.h"
#include "MediaStreamWidgets.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MediaStreamCustomization"

namespace UE::MediaStreamEditor
{

void FMediaStreamCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	// Get objects we are editing.
	TArray<TWeakObjectPtr<UObject>> Objects;
	InDetailBuilder.GetObjectsBeingCustomized(Objects);
	MediaStreamsList.Reserve(Objects.Num());

	for (TWeakObjectPtr<UObject>& Obj : Objects)
	{
		TWeakObjectPtr<UMediaStream> MediaStream = Cast<UMediaStream>(Obj.Get());
		if (MediaStream.IsValid())
		{
			MediaStreamsList.Add(MediaStream);
		}
	}

	if (MediaStreamsList.IsEmpty())
	{
		return;
	}

	bool bHasValidPlayer = false;

	if (TSharedPtr<IPropertyHandle> PlayerHandle = InDetailBuilder.GetProperty(UMediaStream::GetPlayerPropertyName()))
	{
		UObject* Player;

		if (PlayerHandle->GetValue(Player) != FPropertyAccess::Result::Fail)
		{
			if (Player)
			{
				bHasValidPlayer = true;
			}
		}
	}

	if (bHasValidPlayer)
	{
		AddControlCategory(InDetailBuilder);
	}

	AddSourceCategory(InDetailBuilder);

	if (bHasValidPlayer)
	{
		AddDetailsCategory(InDetailBuilder);
		AddTextureCategory(InDetailBuilder);
		AddCacheCategory(InDetailBuilder);
		AddPlayerCategory(InDetailBuilder);
	}
}

void FMediaStreamCustomization::AddControlCategory(IDetailLayoutBuilder& InDetailBuilder)
{
	UMediaStream* MediaStream = GetMediaStream();

	if (!MediaStream)
	{
		return;
	}

	IDetailCategoryBuilder& ControlsCategory = InDetailBuilder.EditCategory("Media Controls");

	// Add media player playback slider
	ControlsCategory.AddCustomRow(LOCTEXT("Track", "Track"))
		.IsEnabled(TAttribute<bool>::CreateSPLambda(this, 
			[this]()
			{
				if (UMediaStream* MediaStream = GetMediaStream())
				{
					return !FMediaStreamEditorSequencerLibrary::HasTrack(MediaStream);
				}

				return false;
			}))
		[
			FMediaStreamWidgets::CreateTrackWidget({MediaStream})
		];

	// Add media control buttons.
	ControlsCategory.AddCustomRow(LOCTEXT("Controls", "Controls"))
	[
		FMediaStreamWidgets::CreateControlsWidget({MediaStream})
	];

	TSharedPtr<IPropertyHandle> PlayerHandle = InDetailBuilder.GetProperty(UMediaStream::GetPlayerPropertyName());
	TSharedPtr<IPropertyHandle> PlayerConfigHandle = PlayerHandle->GetChildHandle("PlayerConfig");

	if (!PlayerConfigHandle.IsValid())
	{
		return;
	}

	if (TSharedPtr<IPropertyHandle> PlayerOnOpenHandle = PlayerHandle->GetChildHandle("bPlayOnOpen"))
	{
		ControlsCategory.AddProperty(PlayerOnOpenHandle);
	}

	if (TSharedPtr<IPropertyHandle> LoopingHandle = PlayerHandle->GetChildHandle("bLooping"))
	{
		ControlsCategory.AddProperty(LoopingHandle);
	}
}

void FMediaStreamCustomization::AddSourceCategory(IDetailLayoutBuilder& InDetailBuilder)
{
	TSharedPtr<IPropertyHandle> SourceHandle = InDetailBuilder.GetProperty(UMediaStream::GetSourcePropertyName());

	if (!SourceHandle.IsValid())
	{
		return;
	}

	SourceHandle->MarkHiddenByCustomization();

	IDetailCategoryBuilder& SourceCategory = InDetailBuilder.EditCategory("Media Source");

	TArray<UObject*> Outers;
	SourceHandle->GetOuterObjects(Outers);

	if (Outers.IsEmpty())
	{
		return;
	}

	UMediaStream* MediaStream = Cast<UMediaStream>(Outers[0]);

	if (!MediaStream)
	{
		return;
	}

	const IMediaStreamSchemeHandler::FCustomWidgets WidgetRows = FMediaStreamWidgets::GenerateSourceSchemeRows(MediaStream);

	if (WidgetRows.CustomRows.IsEmpty())
	{
		return;
	}

	for (const IMediaStreamSchemeHandler::FWidgetRow& WidgetRow : WidgetRows.CustomRows)
	{
		if (!WidgetRow.SourceProperty)
		{
			continue;
		}

		TSharedPtr<IPropertyHandle> ChildHandle = SourceHandle->GetChildHandle(WidgetRow.SourceProperty->GetFName());
		
		SourceCategory.AddProperty(ChildHandle)
			.IsEnabled(WidgetRow.Enabled)
			.Visibility(WidgetRow.Visibility)
			.DisplayName(WidgetRow.Name)
			.CustomWidget()
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(WidgetRow.Name)
			]
			.ValueContent()
			[
				WidgetRow.Widget
			];
	}
}

void FMediaStreamCustomization::AddDetailsCategory(IDetailLayoutBuilder& InDetailBuilder)
{
	IDetailCategoryBuilder& MediaDetailsCategory = InDetailBuilder.EditCategory("Media Details");

	MediaDetailsCategory.AddCustomRow(LOCTEXT("Details", "Details"))
		[
			FMediaStreamWidgets::CreateTextureDetailsWidget(GetMediaStream())
		];
}

void FMediaStreamCustomization::AddTextureCategory(IDetailLayoutBuilder& InDetailBuilder)
{
	TSharedPtr<IPropertyHandle> PlayerHandle = InDetailBuilder.GetProperty(UMediaStream::GetPlayerPropertyName());
	TSharedPtr<IPropertyHandle> MediaTextureHandle = PlayerHandle->GetChildHandle("MediaTexture");
	TSharedPtr<IPropertyHandle> TextureConfigHandle = PlayerHandle->GetChildHandle("TextureConfig");

	if (!MediaTextureHandle.IsValid() && !TextureConfigHandle.IsValid())
	{
		return;
	}

	IDetailCategoryBuilder& MediaTextureCategory = InDetailBuilder.EditCategory("Media Texture");

	if (MediaTextureHandle.IsValid())
	{
		MediaTextureHandle->MarkHiddenByCustomization();
		MediaTextureCategory.AddProperty(MediaTextureHandle);
	}

	if (TextureConfigHandle.IsValid())
	{
		TextureConfigHandle->MarkHiddenByCustomization();
		MediaTextureCategory.AddProperty(TextureConfigHandle);
	}
}

void FMediaStreamCustomization::AddCacheCategory(IDetailLayoutBuilder& InDetailBuilder)
{
	TSharedPtr<IPropertyHandle> PlayerHandle = InDetailBuilder.GetProperty(UMediaStream::GetPlayerPropertyName());
	TSharedPtr<IPropertyHandle> PlayerConfigHandle = PlayerHandle->GetChildHandle("PlayerConfig");

	if (!PlayerConfigHandle.IsValid())
	{
		return;
	}

	TSharedPtr<IPropertyHandle> UseLocalCacheHandle = PlayerConfigHandle->GetChildHandle("bUseLocalCache");
	TSharedPtr<IPropertyHandle> CacheAheadHandle = PlayerConfigHandle->GetChildHandle("CacheAhead");
	TSharedPtr<IPropertyHandle> CacheBehindHandle = PlayerConfigHandle->GetChildHandle("CacheBehind");
	TSharedPtr<IPropertyHandle> CacheBehindGameHandle = PlayerConfigHandle->GetChildHandle("CacheBehindGame");

	if (!UseLocalCacheHandle.IsValid() && !CacheAheadHandle.IsValid() && !CacheBehindHandle.IsValid() && !CacheBehindGameHandle.IsValid())
	{
		return;
	}

	IDetailCategoryBuilder& MediaCacheCategory = InDetailBuilder.EditCategory("Media Cache");

	if (UseLocalCacheHandle.IsValid())
	{
		UseLocalCacheHandle->MarkHiddenByCustomization();
		MediaCacheCategory.AddProperty(UseLocalCacheHandle);
	}

	if (CacheAheadHandle.IsValid())
	{
		CacheAheadHandle->MarkHiddenByCustomization();
		MediaCacheCategory.AddProperty(CacheAheadHandle);
	}

	if (CacheBehindHandle.IsValid())
	{
		CacheBehindHandle->MarkHiddenByCustomization();
		MediaCacheCategory.AddProperty(CacheBehindHandle);
	}

	if (CacheBehindGameHandle.IsValid())
	{
		CacheBehindGameHandle->MarkHiddenByCustomization();
		MediaCacheCategory.AddProperty(CacheBehindGameHandle);
	}
}

void FMediaStreamCustomization::AddPlayerCategory(IDetailLayoutBuilder& InDetailBuilder)
{
	TSharedPtr<IPropertyHandle> PlayerHandle = InDetailBuilder.GetProperty(UMediaStream::GetPlayerPropertyName());
	PlayerHandle->MarkHiddenByCustomization();

	IDetailCategoryBuilder& MediaPlayerCategory = InDetailBuilder.EditCategory("Media Player");

	const TArray<FName> PropertyNames = {
		"RequestedSeekFrame",
		"PlaybackState",
		"PlaylistIndex",
		"PlayerConfig",
		"bReadOnly"
	};

	for (const FName& PropertyName : PropertyNames)
	{
		if (TSharedPtr<IPropertyHandle> PropertyHandle = PlayerHandle->GetChildHandle(PropertyName))
		{
			PropertyHandle->MarkHiddenByCustomization();
			MediaPlayerCategory.AddProperty(PropertyHandle);
		}
	}
}

UMediaStream* FMediaStreamCustomization::GetMediaStream() const
{
	for (const TWeakObjectPtr<UMediaStream>& MediaStreamPtr : MediaStreamsList)
	{
		if (UMediaStream* MediaStream = MediaStreamPtr.Get())
		{
			return MediaStream;
		}
	}

	return nullptr;
}

} // UE::MediaStreamEditor

#undef LOCTEXT_NAMESPACE
