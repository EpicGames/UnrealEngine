// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

#include "Containers/Array.h"
#include "UObject/WeakObjectPtr.h"

class IDetailLayoutBuilder;
class UMediaStream;

namespace UE::MediaStreamEditor
{

class FMediaStreamSourceCustomization;

/**
 * Implements a details view customization for the UMediaStreamComponent class.
 */
class FMediaStreamCustomization : public IDetailCustomization
{
public:
	/**
	 * Creates an instance of this class.
	 * @return The new instance.
	 */
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FMediaStreamCustomization>();
	}

	virtual ~FMediaStreamCustomization() override = default;

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization

private:
	/** List of the media streams we are editing. */
	TArray<TWeakObjectPtr<UMediaStream>> MediaStreamsList;

	/** Returns the first media stream being edited. */
	UMediaStream* GetMediaStream() const;

	/** Adds a scrubbable track and media control buttons */
	void AddControlCategory(IDetailLayoutBuilder& InDetailBuilder);

	/** Adds options for the media source. */
	void AddSourceCategory(IDetailLayoutBuilder& InDetailBuilder);

	/** Adds media and player details. */
	void AddDetailsCategory(IDetailLayoutBuilder& InDetailBuilder);

	/** Adds media texture object and options. */
	void AddTextureCategory(IDetailLayoutBuilder& InDetailBuilder);

	/** Adds media cache settings. */
	void AddCacheCategory(IDetailLayoutBuilder& InDetailBuilder);

	/** Adds player config options. */
	void AddPlayerCategory(IDetailLayoutBuilder& InDetailBuilder);
};

} // UE::MediaStreamEditor
