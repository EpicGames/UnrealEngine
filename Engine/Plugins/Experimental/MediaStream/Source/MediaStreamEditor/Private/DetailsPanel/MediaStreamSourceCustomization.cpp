// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsPanel/MediaStreamSourceCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "IDetailGroup.h"
#include "IMediaStreamSchemeHandler.h"
#include "MediaStream.h"
#include "MediaStreamSchemeHandlerManager.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MediaStreamSourceCustomization"

namespace UE::MediaStreamEditor
{

IMediaStreamSchemeHandler::FCustomWidgets FMediaStreamSourceCustomization::Customize(UMediaStream* InMediaStream)
{
	if (!IsValid(InMediaStream))
	{
		return {};
	}

	MediaStream = InMediaStream;

	IMediaStreamSchemeHandler::FCustomWidgets CustomWidgets;

	AddSourceSchemeSelector(CustomWidgets);
	AddSchemeCustomizations(CustomWidgets);

	return CustomWidgets;
}

void FMediaStreamSourceCustomization::AddSourceSchemeSelector(IMediaStreamSchemeHandler::FCustomWidgets& InOutCustomWidgets)
{
	TSharedRef<SGridPanel> SchemeSelector = MakeShared<SGridPanel>();

	AddSourceSchemeSelectRow(SchemeSelector, 0, TEXT("None"));
	AddSourceSchemeSelectRow(SchemeSelector, 1, TEXT("File"));
	AddSourceSchemeSelectRow(SchemeSelector, 2, TEXT("Asset"));

	/**
	 * Later implementation
	 *
	const TArray<FName> Schemes = FMediaStreamSchemeHandlerManager::Get().GetSchemeHandlerNames();
	int32 Row = 1;

	for (const FName& Scheme : Schemes)
	{
		AddSourceSchemeSelectRow(SchemeSelector, Row, Scheme);
		++Row;
	}
	 */

	InOutCustomWidgets.CustomRows.Add({
		LOCTEXT("SourceSchemeName", "Scheme"),
		SchemeSelector,
		/* Enabled */ true,
		EVisibility::Visible,
		FMediaStreamSource::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FMediaStreamSource, Scheme))
	});
}

void FMediaStreamSourceCustomization::AddSourceSchemeSelectRow(TSharedRef<SGridPanel> InContainer, int32 InRow, const FName& InName)
{
	TWeakObjectPtr<UMediaStream> MediaStreamWeak = MediaStream;

	InContainer->AddSlot(0, InRow)
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.VAlign(EVerticalAlignment::VAlign_Center)
		.Padding(5.f, 2.f)
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "Menu.RadioButton")
			.IsChecked_Static(&FMediaStreamSourceCustomization::GetSourceCheckBoxState, MediaStreamWeak, InName)
			.OnCheckStateChanged_Static(&FMediaStreamSourceCustomization::OnSourceCheckBoxStateChanged, MediaStreamWeak, InName)
		];

	InContainer->AddSlot(1, InRow)
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.VAlign(EVerticalAlignment::VAlign_Center)
		.Padding(0.f, 2.f, 5.f, 2.f)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ContentPadding(FMargin(5.f, 1.f))
			.ButtonStyle(FAppStyle::Get(), TEXT("NoBorder"))
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(FText::FromName(InName))
			]
		];
}

ECheckBoxState FMediaStreamSourceCustomization::GetSourceCheckBoxState(TWeakObjectPtr<UMediaStream> InMediaStreamWeak, FName InSourceScheme)
{
	if (UMediaStream* MediaStream = InMediaStreamWeak.Get())
	{
		if (MediaStream->GetSource().Scheme == InSourceScheme )
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}

void FMediaStreamSourceCustomization::OnSourceCheckBoxStateChanged(ECheckBoxState InState, TWeakObjectPtr<UMediaStream> InMediaStreamWeak, FName InSourceScheme)
{
	if (UMediaStream* MediaStream = InMediaStreamWeak.Get())
	{
		FMediaStreamSchemeHandlerLibrary::SetSource(MediaStream, InSourceScheme, "");
	}
}

void FMediaStreamSourceCustomization::AddSchemeCustomizations(IMediaStreamSchemeHandler::FCustomWidgets& InOutCustomWidgets)
{
	const FMediaStreamSchemeHandlerManager& Manager = FMediaStreamSchemeHandlerManager::Get();
	const TArray<FName> Schemes = Manager.GetSchemeHandlerNames();

	for (const FName& Scheme : Schemes)
	{
		Manager.GetHandlerTypeForScheme(Scheme)->CreatePropertyCustomization(MediaStream, InOutCustomWidgets);
	}
}

} // UE::MediaStreamEditor

#undef LOCTEXT_NAMESPACE
