// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

#include "IMediaStreamSchemeHandler.h"
#include "Templates/SharedPointer.h"

class FReply;
class SGridPanel;
class SWidget;
class UMediaPlayer;
class UMediaStream;
enum class ECheckBoxState : uint8;
struct FMediaStreamSource;

namespace UE::MediaStreamEditor
{

/**
 * Generates custom widgets for source scheme types.
 * Object should not last behind the local scope.
 */
class FMediaStreamSourceCustomization : public TSharedFromThis<FMediaStreamSourceCustomization>
{
public:
	virtual ~FMediaStreamSourceCustomization() = default;

	/** Produces widgets for scheme customizations */
	IMediaStreamSchemeHandler::FCustomWidgets Customize(UMediaStream* InMediaStream);

private:
	/** Whether a particular scheme is active. */
	static ECheckBoxState GetSourceCheckBoxState(TWeakObjectPtr<UMediaStream> InMediaStreamWeak, FName InSourceScheme);

	/** Changes the source to a new scheme. */
	static void OnSourceCheckBoxStateChanged(ECheckBoxState InState, TWeakObjectPtr<UMediaStream> InMediaStreamWeak, FName InSourceScheme);

	/** Temporarily stored media stream. */
	UMediaStream* MediaStream = nullptr;

	/** Adds the scheme selector widget. @see AddSourceSchemeSelectRow. */
	void AddSourceSchemeSelector(IMediaStreamSchemeHandler::FCustomWidgets& InOutCustomWidgets);

	/** Adds the widget for a particular scheme type. */
	void AddSourceSchemeSelectRow(TSharedRef<SGridPanel> InContainer, int32 InRow, const FName& InName);

	/** Adds a customization for a particular scheme. */
	void AddSchemeCustomizations(IMediaStreamSchemeHandler::FCustomWidgets& InOutCustomWidgets);
};

} // UE::MediaStreamEditor
