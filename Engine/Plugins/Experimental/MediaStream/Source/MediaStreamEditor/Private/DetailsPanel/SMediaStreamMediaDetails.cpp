// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsPanel/SMediaStreamMediaDetails.h"

#include "IMediaStreamPlayer.h"
#include "Internationalization/Text.h"
#include "MediaPlayer.h"
#include "MediaStream.h"
#include "Styling/AppStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMediaStreamMediaDetails"

namespace UE::MediaStreamEditor
{

void SMediaStreamMediaDetails::Construct(const FArguments& InArgs, UMediaStream* InMediaStream)
{
	MediaStreamWeak = InMediaStream;

	const ISlateStyle& SlateStyle = FAppStyle::Get();
	const FName StyleName = TEXT("SmallText");

	ChildSlot
	[
		SNew(SHorizontalBox)
		.Visibility(this, &SMediaStreamMediaDetails::AreDetailsVisible)

		// Left side.
		+ SHorizontalBox::Slot()
		[
			SNew(SVerticalBox)

			// Player name.
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(MediaPlayerName, STextBlock)
				.TextStyle(SlateStyle, StyleName)
			]

			// Resolution.
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(ResolutionText, STextBlock)
				.TextStyle(SlateStyle, StyleName)
			]

			// Frame rate.
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(FrameRateText, STextBlock)
				.TextStyle(SlateStyle, StyleName)
			]

			// Resource size.
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(ResourceSizeText, STextBlock)
				.TextStyle(SlateStyle, StyleName)
			]

			// Method.
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(MethodText, STextBlock)
				.TextStyle(SlateStyle, StyleName)
			]
		]

		// Right side.
		+ SHorizontalBox::Slot()
		[
			SNew(SVerticalBox)

			// Format.
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(FormatText, STextBlock)
				.TextStyle(SlateStyle, StyleName)
			]

			// LOD bias.
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(LODBiasText, STextBlock)
				.TextStyle(SlateStyle, StyleName)
			]

			// Num mips.
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(NumMipsText, STextBlock)
				.TextStyle(SlateStyle, StyleName)
			]

			// Num tiles.
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SAssignNew(NumTilesText, STextBlock)
				.TextStyle(SlateStyle, StyleName)
			]
		]

	];

	UpdateDetails();
}

void SMediaStreamMediaDetails::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	UpdateDetails();
}

UMediaPlayer* SMediaStreamMediaDetails::GetMediaPlayer() const
{
	UMediaStream* MediaStream = MediaStreamWeak.Get();

	if (!MediaStream)
	{
		return nullptr;
	}

	IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface();

	if (!MediaStreamPlayer)
	{
		return nullptr;
	}

	return MediaStreamPlayer->GetPlayer();
}

UMediaTexture* SMediaStreamMediaDetails::GetMediaTexture() const
{
	UMediaStream* MediaStream = MediaStreamWeak.Get();

	if (!MediaStream)
	{
		return nullptr;
	}

	IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface();

	if (!MediaStreamPlayer)
	{
		return nullptr;
	}

	return MediaStreamPlayer->GetMediaTexture();
}

EVisibility SMediaStreamMediaDetails::AreDetailsVisible() const
{
	return (!!GetMediaPlayer() || !!GetMediaTexture())
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

void SMediaStreamMediaDetails::UpdateDetails()
{
	FName PlayerName;
	FString Format;
	float FrameRate = 0.0f;
	int32 LODBias = 0;
	FText Method;
	int32 NumMips = 0;
	int32 NumTotalTiles = 0;
	int64 ResourceSize = 0;
	int32 SurfaceWidth = 0;
	int32 SurfaceHeight = 0;

	UMediaPlayer* MediaPlayer = GetMediaPlayer();
	UMediaTexture* MediaTexture = GetMediaTexture();

	if (!MediaPlayer && !MediaTexture)
	{
		return;
	}

	if (MediaPlayer)
	{
		PlayerName = MediaPlayer->GetPlayerName();
		FrameRate = MediaPlayer->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);
		Format = MediaPlayer->GetVideoTrackType(INDEX_NONE, INDEX_NONE);
		FIntPoint NumTiles(EForceInit::ForceInitToZero);
		MediaPlayer->GetMediaInfo<FIntPoint>(NumTiles, UMediaPlayer::MediaInfoNameSourceNumTiles.Resolve());
		NumTotalTiles = NumTiles.X * NumTiles.Y;
	}

	if (MediaTexture)
	{
		LODBias = MediaTexture->GetCachedLODBias();
		Method = MediaTexture->IsCurrentlyVirtualTextured() ?
			LOCTEXT("MethodVirtualStreamed", "Virtual Streamed")
			: (!MediaTexture->IsStreamable() ? LOCTEXT("QuickInfo_MethodNotStreamed", "Not Streamed")
				: LOCTEXT("MethodStreamed", "Streamed"));
		NumMips = MediaTexture->GetTextureNumMips();
		ResourceSize = (MediaTexture->GetResourceSizeBytes(EResourceSizeMode::Exclusive) + 512) / 1024;
		SurfaceWidth = static_cast<int32>(MediaTexture->GetSurfaceWidth());
		SurfaceHeight = static_cast<int32>(MediaTexture->GetSurfaceHeight());
	}

	// Update text.
	MediaPlayerName->SetText(FText::Format(LOCTEXT("Player", "Player: {0}"),
		FText::FromName(PlayerName)));

	FormatText->SetText(FText::Format(LOCTEXT("Format", "Format: {0}"),
		FText::FromString(Format)));

	FrameRateText->SetText(FText::Format(LOCTEXT("FrameRate", "Frame Rate: {0}"), 
		FText::AsNumber(FrameRate)));

	LODBiasText->SetText(FText::Format(LOCTEXT("LODBias", "Combined LOD Bias: {0}"),
		FText::AsNumber(LODBias)));

	MethodText->SetText(FText::Format(LOCTEXT("Method", "Method: {0}"), Method));

	NumMipsText->SetText(FText::Format(LOCTEXT("NumberOfMips", "Mips: {0}"),
		FText::AsNumber(NumMips)));

	NumTilesText->SetText(FText::Format(LOCTEXT("NumberOfTiles", "Tiles: {0}"),
		FText::AsNumber(NumTotalTiles)));

	ResolutionText->SetText(FText::Format(LOCTEXT("Resolution", "Resolution: {0}x{1}"),
		FText::AsNumber(SurfaceWidth), FText::AsNumber(SurfaceHeight)));

	ResourceSizeText->SetText(FText::Format(LOCTEXT("ResourceSize", "Resource Size: {0} KB"),
		FText::AsNumber(ResourceSize)));
}

} // UE::MediaStreamEditor

#undef LOCTEXT_NAMESPACE
