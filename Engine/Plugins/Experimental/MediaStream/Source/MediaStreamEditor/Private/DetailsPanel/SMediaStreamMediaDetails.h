// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "UObject/WeakObjectPtr.h"

class SBox;
class STextBlock;
class UMediaPlayer;
class UMediaStream;
class UMediaTexture;

namespace UE::MediaStreamEditor
{

/**
 * Displays details about the media texture and its player.
 */
class SMediaStreamMediaDetails : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SMediaStreamMediaDetails)
		{}
	SLATE_END_ARGS()

	MEDIASTREAMEDITOR_API void Construct(const FArguments& InArgs, UMediaStream* InMediaStream);

	//~ Begin SWidget
	MEDIASTREAMEDITOR_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget

protected:
	TWeakObjectPtr<UMediaStream> MediaStreamWeak;

	/** Our widgets. */
	TSharedPtr<STextBlock> MediaPlayerName;
	TSharedPtr<STextBlock> FormatText;
	TSharedPtr<STextBlock> FrameRateText;
	TSharedPtr<STextBlock> LODBiasText;
	TSharedPtr<STextBlock> MethodText;
	TSharedPtr<STextBlock> NumMipsText;
	TSharedPtr<STextBlock> NumTilesText;
	TSharedPtr<STextBlock> ResolutionText;
	TSharedPtr<STextBlock> ResourceSizeText;

	/** Retrieves the media player. */
	UMediaPlayer* GetMediaPlayer() const;

	/** Retrieves the media texture. */
	UMediaTexture* GetMediaTexture() const;

	/** Determines whether the entire widget should be visible. */
	EVisibility AreDetailsVisible() const;

	/** Updates our widgets to reflect the current state. */
	void UpdateDetails();
};

} // UE::MediaStreamEditor
