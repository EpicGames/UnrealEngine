// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsPanel/SMediaStreamMediaTrack.h"

#include "IMediaStreamPlayer.h"
#include "MediaPlayer.h"
#include "MediaPlayerEditorModule.h"
#include "MediaStream.h"
#include "Modules/ModuleManager.h"
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SBox.h"

namespace UE::MediaStreamEditor
{

void SMediaStreamMediaTrack::Construct(const FArguments& InArgs, const TArray<UMediaStream*>& InMediaStreams)
{
	MediaStreamsWeak.Reserve(InMediaStreams.Num());

	for (UMediaStream* MediaStream : InMediaStreams)
	{
		MediaStreamsWeak.Add(MediaStream);
	}

	SetCanTick(true);

	ChildSlot
	[
		SAssignNew(Content, SBox)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			CreateTrack()
		]
	];
}

void SMediaStreamMediaTrack::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(InAllottedGeometry, InCurrentTime, InDeltaTime);

	TArray<UMediaPlayer*> MediaPlayers = GetMediaPlayers();
	bool bNeedsUpdate = MediaPlayers.Num() != MediaPlayersWeak.Num();

	if (!bNeedsUpdate)
	{
		for (int32 Index = 0; Index < MediaPlayersWeak.Num(); ++Index)
		{
			UMediaPlayer* MediaPlayer = MediaPlayersWeak[Index].Get();

			if (MediaPlayer != MediaPlayers[Index])
			{
				bNeedsUpdate = true;
				break;
			}
		}
	}

	if (!bNeedsUpdate)
	{
		return;
	}

	Content->SetContent(CreateTrack());
}

TArray<UMediaPlayer*> SMediaStreamMediaTrack::GetMediaPlayers() const
{
	TArray<UMediaPlayer*> MediaPlayers;
	MediaPlayers.Reserve(MediaStreamsWeak.Num());

	for (const TWeakObjectPtr<UMediaStream>& MediaStreamWeak : MediaStreamsWeak)
	{
		if (UMediaStream* MediaStream = MediaStreamWeak.Get())
		{
			if (IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface())
			{
				if (UMediaPlayer* MediaPlayer = MediaStreamPlayer->GetPlayer())
				{
					MediaPlayers.Add(MediaPlayer);
				}
			}
		}
	}

	return MediaPlayers;
}

TSharedRef<SWidget> SMediaStreamMediaTrack::CreateTrack()
{
	TArray<UMediaPlayer*> MediaPlayers = GetMediaPlayers();

	if (MediaPlayers.IsEmpty())
	{
		return SNullWidget::NullWidget;
	}

	IMediaPlayerEditorModule* MediaPlayerEditorModule = FModuleManager::LoadModulePtr<IMediaPlayerEditorModule>("MediaPlayerEditor");

	if (!MediaPlayerEditorModule)
	{
		return SNullWidget::NullWidget;
	}

	MediaPlayersWeak.Empty(MediaPlayers.Num());

	for (UMediaPlayer* MediaPlayer : MediaPlayers)
	{
		MediaPlayersWeak.Add(MediaPlayer);
	}

	const TSharedRef<IMediaPlayerSlider> MediaPlayerSlider = MediaPlayerEditorModule->CreateMediaPlayerSliderWidget(MediaPlayersWeak);

	MediaPlayerSlider->SetSliderHandleColor(FSlateColor(EStyleColor::AccentBlue));
	MediaPlayerSlider->SetVisibleWhenInactive(EVisibility::Visible);

	return MediaPlayerSlider;
}

} // UE::MediaStreamEditor
