// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "UObject/WeakObjectPtr.h"

class SBox;
class UMediaPlayer;
class UMediaStream;

namespace UE::MediaStreamEditor
{

/**
 * Shows a scrubbable track for controlling media playback position.
 */
class SMediaStreamMediaTrack : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SMediaStreamMediaTrack)
		{}
	SLATE_END_ARGS()

	MEDIASTREAMEDITOR_API void Construct(const FArguments& InArgs, const TArray<UMediaStream*>& InMediaStreams);

	//~ Begin SWidget
	MEDIASTREAMEDITOR_API virtual void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget

private:
	/** The media stream for this track. */
	TArray<TWeakObjectPtr<UMediaStream>> MediaStreamsWeak;

	/** The active media player. If this changes, recreate the track. */
	TArray<TWeakObjectPtr<UMediaPlayer>> MediaPlayersWeak;

	/** The container for the track. */
	TSharedPtr<SBox> Content;

	/** Returns the currently active media player. May different from @see MediaPlayerWeak. */
	TArray<UMediaPlayer*> GetMediaPlayers() const;

	/** Creates the track widget. */
	TSharedRef<SWidget> CreateTrack();
};

} // UE::MediaStreamEditor
