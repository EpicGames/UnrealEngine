// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsPanel/SMediaStreamPlaybackControls.h"

#include "DetailLayoutBuilder.h"
#include "IMediaStreamPlayer.h"
#include "MediaPlayer.h"
#include "MediaStream.h"
#include "MediaStreamEditorSequencerLibrary.h"
#include "MediaStreamEditorStyle.h"
#include "MediaStreamEnums.h"
#include "MediaStreamPlayerConfig.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMediaStreamPlaybackControls"

namespace UE::MediaStreamEditor
{

void SMediaStreamPlaybackControls::Construct(const FArguments& InArgs, const TArray<UMediaStream*>& InMediaStreams)
{
	MediaStreamsWeak.Reserve(InMediaStreams.Num());

	for (UMediaStream* MediaStream : InMediaStreams)
	{
		MediaStreamsWeak.Add(MediaStream);
	}

	const ISlateStyle* MediaStreamStyle = &FMediaStreamEditorStyle::Get().Get();

	ChildSlot
	[
		SNew(SWrapBox)
		.Orientation(Orient_Horizontal)
		.UseAllottedSize(true)
		.HAlign(HAlign_Center)
		.InnerSlotPadding(FVector2D::ZeroVector)

		// Open button.
		+ SWrapBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.Visibility(this, &SMediaStreamPlaybackControls::Open_GetVisibility)
			.OnClicked(this, &SMediaStreamPlaybackControls::Open_OnClicked)
			.ButtonStyle(FMediaStreamEditorStyle::Get(), "MediaButtons")
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(MediaStreamStyle->GetBrush("MediaStreamEditor.OpenMedia.Small"))
				.ToolTipText(LOCTEXT("Open Media", "Opens the current media, if the source is valid."))
			]
		]

		// Close button.
		+ SWrapBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.Visibility(this, &SMediaStreamPlaybackControls::Close_GetVisibility)
			.OnClicked(this, &SMediaStreamPlaybackControls::Close_OnClicked)
			.ButtonStyle(FMediaStreamEditorStyle::Get(), "MediaButtons")
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(MediaStreamStyle->GetBrush("MediaStreamEditor.CloseMedia.Small"))
				.ToolTipText(LOCTEXT("Close Media", "Closes the currently opened media."))
			]
		]

		// Rewind button.
		+ SWrapBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaStreamPlaybackControls::Rewind_IsEnabled)
			.OnClicked(this, &SMediaStreamPlaybackControls::Rewind_OnClicked)
			.ButtonStyle(FMediaStreamEditorStyle::Get(), "MediaButtons")
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(MediaStreamStyle->GetBrush("MediaStreamEditor.RewindMedia.Small"))
				.ToolTipText(LOCTEXT("Rewind", "Rewind the media to the beginning"))
			]
		]

		// Reverse button.
		+ SWrapBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaStreamPlaybackControls::Reverse_IsEnabled)
			.OnClicked(this, &SMediaStreamPlaybackControls::Reverse_OnClicked)
			.ButtonStyle(FMediaStreamEditorStyle::Get(), "MediaButtons")
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(MediaStreamStyle->GetBrush("MediaStreamEditor.ReverseMedia.Small"))
				.ToolTipText(LOCTEXT("Reverse", "Reverse media playback"))
			]
		]

		// Play button.
		+ SWrapBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.Visibility(this, &SMediaStreamPlaybackControls::Play_GetVisibility)
			.IsEnabled(this, &SMediaStreamPlaybackControls::Play_IsEnabled)
			.OnClicked(this, &SMediaStreamPlaybackControls::Play_OnClicked)
			.ButtonStyle(FMediaStreamEditorStyle::Get(), "MediaButtons")
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(MediaStreamStyle->GetBrush("MediaStreamEditor.PlayMedia.Small"))
				.ToolTipText(LOCTEXT("Play", "Start media playback"))
			]
		]

		// Pause button.
		+ SWrapBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.Visibility(this, &SMediaStreamPlaybackControls::Pause_GetVisibility)
			.IsEnabled(this, &SMediaStreamPlaybackControls::Pause_IsEnabled)
			.OnClicked(this, &SMediaStreamPlaybackControls::Pause_OnClicked)
			.ButtonStyle(FMediaStreamEditorStyle::Get(), "MediaButtons")
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(MediaStreamStyle->GetBrush("MediaStreamEditor.PauseMedia.Small"))
				.ToolTipText(LOCTEXT("Pause", "Pause media playback"))
			]
		]

		// Forward button.
		+ SWrapBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaStreamPlaybackControls::Forward_IsEnabled)
			.OnClicked(this, &SMediaStreamPlaybackControls::Forward_OnClicked)
			.ButtonStyle(FMediaStreamEditorStyle::Get(), "MediaButtons")
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(MediaStreamStyle->GetBrush("MediaStreamEditor.ForwardMedia.Small"))
				.RenderTransform(FSlateRenderTransform(TTransform2<float>(TQuat2<float>(PI), FVector2f(20.f, 20.f))))
				.ToolTipText(LOCTEXT("Forward", "Fast forward media playback"))
			]
		]

		// Add Media Track
		+ SWrapBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ContentPadding(FMargin(4.f, 4.f, 4.f, 3.f))
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaStreamPlaybackControls::AddTrack_GetEnabled)
			.OnClicked(this, &SMediaStreamPlaybackControls::AddTrack_OnClicked)
			.ButtonStyle(FMediaStreamEditorStyle::Get(), "MediaButtons")
			.ToolTipText(this, &SMediaStreamPlaybackControls::AddTrack_GetToolTip)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("AddTrack", "Add Track"))
			]
		]
	];
}

TArray<UMediaStream*> SMediaStreamPlaybackControls::GetMediaStreams() const
{
	TArray<UMediaStream*> MediaStreams;
	MediaStreams.Reserve(MediaStreamsWeak.Num());

	for (const TWeakObjectPtr<UMediaStream>& MediaStreamWeak : MediaStreamsWeak)
	{
		if (UMediaStream* MediaStream = MediaStreamWeak.Get())
		{
			MediaStreams.Add(MediaStream);
		}
	}

	return MediaStreams;
}

TArray<UMediaPlayer*> SMediaStreamPlaybackControls::GetMediaPlayers() const
{
	TArray<UMediaStream*> MediaStreams = GetMediaStreams();
	TArray<UMediaPlayer*> MediaPlayers;
	MediaPlayers.Reserve(MediaStreams.Num());

	for (UMediaStream* MediaStream : MediaStreams)
	{
		if (IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface())
		{
			if (UMediaPlayer* MediaPlayer = MediaStreamPlayer->GetPlayer())
			{
				MediaPlayers.Add(MediaPlayer);
			}
		}
	}

	return MediaPlayers;
}

void SMediaStreamPlaybackControls::OnChangePlaybackState(EMediaStreamPlaybackState InState)
{
	for (UMediaStream* MediaStream : GetMediaStreams())
	{
		if (FMediaStreamEditorSequencerLibrary::HasTrack(MediaStream))
		{
			continue;
		}
		
		if (IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface())
		{
			if (MediaStreamPlayer->IsReadOnly())
			{
				continue;
			}

			switch (InState)
			{
				case EMediaStreamPlaybackState::Play:
				{
					const float RequestedRate = FMath::Abs(MediaStreamPlayer->GetPlayerConfig().Rate);

					if (RequestedRate != MediaStreamPlayer->GetPlayerConfig().Rate)
					{
						FMediaStreamPlayerConfig PlayerConfig = MediaStreamPlayer->GetPlayerConfig();
						PlayerConfig.Rate = RequestedRate;

						MediaStreamPlayer->SetPlayerConfig(PlayerConfig);
					}

					MediaStreamPlayer->Play();
					break;
				}

				case EMediaStreamPlaybackState::Pause:
					MediaStreamPlayer->Pause();
					break;
			}
		}
	}
}

void SMediaStreamPlaybackControls::OnChangePlaybackDirection(EMediaStreamPlaybackDirection InDirection)
{
	for (UMediaStream* MediaStream : GetMediaStreams())
	{
		if (FMediaStreamEditorSequencerLibrary::HasTrack(MediaStream))
		{
			continue;
		}

		if (IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface())
		{
			if (MediaStreamPlayer->IsReadOnly())
			{
				continue;
			}

			const float RequestedRate = FMath::Abs(MediaStreamPlayer->GetPlayerConfig().Rate)
				* (InDirection == EMediaStreamPlaybackDirection::Forward ? 1.f : -1.f);

			if (RequestedRate != MediaStreamPlayer->GetPlayerConfig().Rate)
			{
				FMediaStreamPlayerConfig PlayerConfig = MediaStreamPlayer->GetPlayerConfig();
				PlayerConfig.Rate = RequestedRate;

				MediaStreamPlayer->SetPlayerConfig(PlayerConfig);
			}
		}
	}
}

void SMediaStreamPlaybackControls::OnPlaybackSeek(EMediaStreamPlaybackSeek InPosition)
{
	for (UMediaStream* MediaStream : GetMediaStreams())
	{
		if (FMediaStreamEditorSequencerLibrary::HasTrack(MediaStream))
		{
			continue;
		}

		if (IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface())
		{
			if (MediaStreamPlayer->IsReadOnly())
			{
				continue;
			}

			switch (InPosition)
			{
				case EMediaStreamPlaybackSeek::Previous:
					MediaStreamPlayer->Previous();
					break;

				case EMediaStreamPlaybackSeek::Start:
					MediaStreamPlayer->Rewind();
					break;

				case EMediaStreamPlaybackSeek::End:
					MediaStreamPlayer->FastForward();
					break;

				case EMediaStreamPlaybackSeek::Next:
					MediaStreamPlayer->Next();
					break;
			}
		}
	}
}

bool SMediaStreamPlaybackControls::IsRateSupported(float InRate) const
{
	for (UMediaPlayer* MediaPlayer : GetMediaPlayers())
	{
		if (MediaPlayer->SupportsRate(InRate, /* Unthinned */ true))
		{
			return true;
		}
	}

	return false;
}

EVisibility SMediaStreamPlaybackControls::Open_GetVisibility() const
{
	for (UMediaStream* MediaStream : GetMediaStreams())
	{
		if (MediaStream->HasValidSource())
		{
			if (IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface())
			{
				if (UMediaPlayer* MediaPlayer = MediaStreamPlayer->GetPlayer())
				{
					if (!MediaPlayer->IsReady())
					{
						return EVisibility::Visible;
					}
				}
				else
				{
					return EVisibility::Visible;
				}
			}
		}
	}

	return EVisibility::Collapsed;
}

FReply SMediaStreamPlaybackControls::Open_OnClicked()
{
	for (UMediaStream* MediaStream : GetMediaStreams())
	{
		if (IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface())
		{
			MediaStreamPlayer->OpenSource();
		}
	}

	return FReply::Handled();
}

EVisibility SMediaStreamPlaybackControls::Close_GetVisibility() const
{
	for (UMediaStream* MediaStream : GetMediaStreams())
	{
		if (MediaStream->HasValidSource())
		{
			if (IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface())
			{
				if (UMediaPlayer* MediaPlayer = MediaStreamPlayer->GetPlayer())
				{
					if (!FMediaStreamEditorSequencerLibrary::HasTrack(MediaStream))
					{
						if (!MediaPlayer->IsClosed())
						{
							return EVisibility::Visible;
						}
					}
				}
			}
		}
	}

	return EVisibility::Collapsed;
}

FReply SMediaStreamPlaybackControls::Close_OnClicked()
{
	for (UMediaStream* MediaStream : GetMediaStreams())
	{
		if (IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface())
		{
			MediaStreamPlayer->Close();
		}
	}

	return FReply::Handled();
}

bool SMediaStreamPlaybackControls::Rewind_IsEnabled() const
{
	for (UMediaStream* MediaStream : GetMediaStreams())
	{
		if (!FMediaStreamEditorSequencerLibrary::HasTrack(MediaStream))
		{
			if (IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface())
			{
				if (UMediaPlayer* MediaPlayer = MediaStreamPlayer->GetPlayer())
				{
					if (MediaPlayer->IsReady() &&
						MediaPlayer->SupportsSeeking() &&
						MediaPlayer->GetTime() > FTimespan::Zero())
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

FReply SMediaStreamPlaybackControls::Rewind_OnClicked()
{
	OnPlaybackSeek(EMediaStreamPlaybackSeek::Previous);
	return FReply::Handled();
}

bool SMediaStreamPlaybackControls::Reverse_IsEnabled() const
{
	for (UMediaStream* MediaStream : GetMediaStreams())
	{
		if (!FMediaStreamEditorSequencerLibrary::HasTrack(MediaStream))
		{
			if (IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface())
			{
				if (UMediaPlayer* MediaPlayer = MediaStreamPlayer->GetPlayer())
				{
					const float Rate = FMath::Abs(MediaPlayer->GetRate());

					if (MediaPlayer->IsReady() && IsRateSupported(Rate > 0 ? -Rate : -1.f))
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

FReply SMediaStreamPlaybackControls::Reverse_OnClicked()
{
	OnChangePlaybackDirection(EMediaStreamPlaybackDirection::Backward);
	return FReply::Handled();
}

EVisibility SMediaStreamPlaybackControls::Play_GetVisibility() const
{
	for (UMediaStream* MediaStream : GetMediaStreams())
	{
		if (!FMediaStreamEditorSequencerLibrary::HasTrack(MediaStream))
		{
			if (IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface())
			{
				if (UMediaPlayer* MediaPlayer = MediaStreamPlayer->GetPlayer())
				{
					if (!MediaPlayer->IsPlaying())
					{
						return EVisibility::Visible;
					}
				}
			}
		}
	}

	return EVisibility::Collapsed;
}

bool SMediaStreamPlaybackControls::Play_IsEnabled() const
{
	for (UMediaStream* MediaStream : GetMediaStreams())
	{
		if (!FMediaStreamEditorSequencerLibrary::HasTrack(MediaStream))
		{
			if (IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface())
			{
				if (UMediaPlayer* MediaPlayer = MediaStreamPlayer->GetPlayer())
				{
					if (MediaPlayer->IsReady()
						&& (!MediaPlayer->IsPlaying() || (MediaPlayer->GetRate() != 1.0f)))
					{
						return true;
					}
				}
			}
		}
	}

	return false; 
}

FReply SMediaStreamPlaybackControls::Play_OnClicked()
{
	OnChangePlaybackState(EMediaStreamPlaybackState::Play);
	return FReply::Handled();
}

EVisibility SMediaStreamPlaybackControls::Pause_GetVisibility() const
{
	for (UMediaStream* MediaStream : GetMediaStreams())
	{
		if (!FMediaStreamEditorSequencerLibrary::HasTrack(MediaStream))
		{
			if (IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface())
			{
				if (UMediaPlayer* MediaPlayer = MediaStreamPlayer->GetPlayer())
				{
					if (MediaPlayer->CanPause() && !MediaPlayer->IsPaused())
					{
						return EVisibility::Visible;
					}
				}
			}
		}
	}

	return EVisibility::Collapsed;
}

bool SMediaStreamPlaybackControls::Pause_IsEnabled() const
{
	for (UMediaStream* MediaStream : GetMediaStreams())
	{
		if (!FMediaStreamEditorSequencerLibrary::HasTrack(MediaStream))
		{
			if (IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface())
			{
				if (UMediaPlayer* MediaPlayer = MediaStreamPlayer->GetPlayer())
				{
					if (MediaPlayer->CanPause() && !MediaPlayer->IsPaused())
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

FReply SMediaStreamPlaybackControls::Pause_OnClicked()
{
	OnChangePlaybackState(EMediaStreamPlaybackState::Pause);
	return FReply::Handled();
}

bool SMediaStreamPlaybackControls::Forward_IsEnabled() const
{
	for (UMediaStream* MediaStream : GetMediaStreams())
	{
		if (!FMediaStreamEditorSequencerLibrary::HasTrack(MediaStream))
		{
			if (IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface())
			{
				if (UMediaPlayer* MediaPlayer = MediaStreamPlayer->GetPlayer())
				{
					if (MediaPlayer->IsReady())
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

FReply SMediaStreamPlaybackControls::Forward_OnClicked()
{
	OnPlaybackSeek(EMediaStreamPlaybackSeek::End);
	return FReply::Handled();
}

bool SMediaStreamPlaybackControls::AddTrack_GetEnabled() const
{
	for (UMediaStream* MediaStream : GetMediaStreams())
	{
		if (!FMediaStreamEditorSequencerLibrary::HasTrack(MediaStream) && FMediaStreamEditorSequencerLibrary::CanAddTrack(MediaStream))
		{
			return true;
		}
	}

	return false;
}

FText SMediaStreamPlaybackControls::AddTrack_GetToolTip() const
{
	bool bCanAddTrack = false;
	bool bHasTrack = false;
	bool bHasAsset = false;

	for (UMediaStream* MediaStream : GetMediaStreams())
	{
		if (!bCanAddTrack && FMediaStreamEditorSequencerLibrary::CanAddTrack(MediaStream))
		{
			bCanAddTrack = true;
		}

		if (!bHasTrack && FMediaStreamEditorSequencerLibrary::HasTrack(MediaStream))
		{
			bHasTrack = true;
		}

		if (!bHasAsset && !MediaStream->GetWorld() && !MediaStream->IsIn(GetTransientPackage()))
		{
			bHasAsset = true;
		}
	}

	TArray<FText> ToolTips;

	if (!bCanAddTrack)
	{
		ToolTips.Add(LOCTEXT("CannotAddTrackToolTip", "Note: A track cannot be added."));
	}

	if (bHasTrack)
	{
		ToolTips.Add(LOCTEXT("HasTrackToolTip", "Note: A track has already been added."));
	}

	if (bHasAsset)
	{
		ToolTips.Add(LOCTEXT("HasAssetToolTip", "Note: You cannot bind assets from the Content Browser to the Level Sequence. Consider creating a Material Designer Instance."));
	}

	if (ToolTips.IsEmpty())
	{
		return LOCTEXT("AddTrackToolTip", "Add Track to Level Sequence");
	}

	return FText::Format(
		LOCTEXT("AddTrackWithNotesToolTip", "Add Track to Level Sequence\n\n{0}"),
		FText::Join(INVTEXT("\n"), ToolTips)
	);
}

FReply SMediaStreamPlaybackControls::AddTrack_OnClicked()
{
	for (UMediaStream* MediaStream : GetMediaStreams())
	{
		if (FMediaStreamEditorSequencerLibrary::CanAddTrack(MediaStream))
		{
			FMediaStreamEditorSequencerLibrary::AddTrack(MediaStream);
		}
	}

	return FReply::Unhandled();
}

} // UE::MediaStreamEditor

#undef LOCTEXT_NAMESPACE
