// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "UObject/WeakObjectPtr.h"

class SBox;
class UMediaPlayer;
class UMediaStream;
enum class EMediaStreamPlaybackDirection : uint8;
enum class EMediaStreamPlaybackSeek : uint8;
enum class EMediaStreamPlaybackState : uint8;

namespace UE::MediaStreamEditor
{

/**
 * Displays media playback controls, such as play, pause, etc.
 */
class SMediaStreamPlaybackControls : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SMediaStreamPlaybackControls)
		{}
	SLATE_END_ARGS()

	MEDIASTREAMEDITOR_API void Construct(const FArguments& InArgs, const TArray<UMediaStream*>& InMediaStreams);

private:
	/** The media stream for these controls. */
	TArray<TWeakObjectPtr<UMediaStream>> MediaStreamsWeak;

	/** Returns the media streams associated with these controls. */
	TArray<UMediaStream*> GetMediaStreams() const;

	/** Returns the media players associated with these controls. */
	TArray<UMediaPlayer*> GetMediaPlayers() const;

	/** Changes the state of selected media streams. */
	void OnChangePlaybackState(EMediaStreamPlaybackState InState);

	/** Changes the state of selected media streams. */
	void OnChangePlaybackDirection(EMediaStreamPlaybackDirection InDirection);

	/** Changes the state of selected media streams. */
	void OnPlaybackSeek(EMediaStreamPlaybackSeek InPosition);

	/** Returns true if the given rate is supported by the media/player. */
	bool IsRateSupported(float InRate) const;

	/** Button delegates. */

	EVisibility Open_GetVisibility() const;
	FReply Open_OnClicked();

	EVisibility Close_GetVisibility() const;
	FReply Close_OnClicked();

	bool Rewind_IsEnabled() const;
	FReply Rewind_OnClicked();

	bool Reverse_IsEnabled() const;
	FReply Reverse_OnClicked();

	EVisibility Play_GetVisibility() const;
	bool Play_IsEnabled() const;
	FReply Play_OnClicked();

	EVisibility Pause_GetVisibility() const;
	bool Pause_IsEnabled() const;
	FReply Pause_OnClicked();

	bool Forward_IsEnabled() const;
	FReply Forward_OnClicked();

	bool AddTrack_GetEnabled() const;
	FText AddTrack_GetToolTip() const;
	FReply AddTrack_OnClicked();
};

} // UE::MediaStreamEditor
