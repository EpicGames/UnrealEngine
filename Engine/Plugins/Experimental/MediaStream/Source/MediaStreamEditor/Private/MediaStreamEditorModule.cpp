// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaStreamEditorModule.h"

#include "DetailsPanel/MediaStreamCustomization.h"
#include "ISequencerModule.h"
#include "MediaStream.h"
#include "MediaStreamEditorStyle.h"
#include "MediaStreamObjectSchema.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"

namespace UE::MediaStream::Private
{
	const FLazyName SequencerModuleName = TEXT("Sequencer");
}

void FMediaStreamEditorModule::StartupModule()
{
	using namespace UE::MediaStream::Private;

	FMediaStreamEditorStyle::Get();

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyModule.RegisterCustomClassLayout(UMediaStream::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&UE::MediaStreamEditor::FMediaStreamCustomization::MakeInstance));

	ISequencerModule& SequencerModule = FModuleManager::Get().GetModuleChecked<ISequencerModule>(SequencerModuleName);

	MediaStreamObjectSchema = MakeShared<FMediaStreamObjectSchema>();
	SequencerModule.RegisterObjectSchema(MediaStreamObjectSchema);
}

void FMediaStreamEditorModule::ShutdownModule()
{
	using namespace UE::MediaStream::Private;

	if (MediaStreamObjectSchema.IsValid())
	{
		if (ISequencerModule* SequencerModule = FModuleManager::Get().GetModulePtr<ISequencerModule>(SequencerModuleName))
		{
			SequencerModule->UnregisterObjectSchema(MediaStreamObjectSchema);
		}
	}
}

IMPLEMENT_MODULE(FMediaStreamEditorModule, MediaStreamEditor);
