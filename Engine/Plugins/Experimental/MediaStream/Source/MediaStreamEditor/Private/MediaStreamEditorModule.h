// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#include "Delegates/IDelegateInstance.h"
#include "Templates/SharedPointer.h"

class FMediaStreamObjectSchema;

/**
 * Media Stream Editor - Content/type agnostic chainable media proxy with media player integration.
 */
class FMediaStreamEditorModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

protected:
	TSharedPtr<FMediaStreamObjectSchema> MediaStreamObjectSchema;
};
