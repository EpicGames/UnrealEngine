// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaStreamEditorSequencerLibrary.h"

#include "ILevelSequenceEditorToolkit.h"
#include "IMediaStreamPlayer.h"
#include "LevelSequence.h"
#include "LevelSequenceEditorBlueprintLibrary.h"
#include "MediaStream.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneMediaSection.h"
#include "MovieSceneMediaTrack.h"
#include "SequencerUtilities.h"
#include "Subsystems/AssetEditorSubsystem.h"

extern UNREALED_API UEditorEngine* GEditor;

#define LOCTEXT_NAMESPACE "MediaStreamEditorSequencerLibrary"

ULevelSequence* FMediaStreamEditorSequencerLibrary::GetLevelSequence()
{
	if (ULevelSequence* FocusedSequence = ULevelSequenceEditorBlueprintLibrary::GetFocusedLevelSequence())
	{
		return FocusedSequence;
	}

	if (ULevelSequence* FocusedSequence = ULevelSequenceEditorBlueprintLibrary::GetCurrentLevelSequence())
	{
		return FocusedSequence;
	}

	return nullptr;
}

bool FMediaStreamEditorSequencerLibrary::HasTrack(UMediaStream* InMediaStream)
{
	if (!InMediaStream)
	{
		return false;
	}

	ULevelSequence* LevelSequence = GetLevelSequence();

	if (!LevelSequence)
	{
		return false;
	}

	UMovieScene* MovieScene = LevelSequence->GetMovieScene();

	if (!MovieScene)
	{
		return false;
	}

	UWorld* World = LevelSequence->GetWorld();

	if (!World)
	{
		return false;
	}

	TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState = MovieSceneHelpers::CreateTransientSharedPlaybackState(World, LevelSequence);
	const FGuid MediaStreamId = LevelSequence->FindBindingFromObject(InMediaStream, SharedPlaybackState);

	if (MediaStreamId.IsValid())
	{
		return !!MovieScene->FindTrack<UMovieSceneMediaTrack>(MediaStreamId);
	}

	return false;
}

bool FMediaStreamEditorSequencerLibrary::CanAddTrack(UMediaStream* InMediaStream)
{
	if (!InMediaStream)
	{
		return false;
	}

	if (InMediaStream->IsIn(GetTransientPackage()))
	{
		return false;
	}

	UObject* MediaObject = InMediaStream->GetSource().Object;

	// Can't add a proxy stream to a track.
	if (!IsValid(MediaObject) || MediaObject->IsA<UMediaStream>())
	{
		return false;
	}

	IMediaStreamPlayer* MediaStreamPlayerIntf = InMediaStream->GetPlayer().GetInterface();

	if (!MediaStreamPlayerIntf || MediaStreamPlayerIntf->IsReadOnly())
	{
		return false;
	}

	UMediaPlayer* MediaPlayer = MediaStreamPlayerIntf->GetPlayer();

	if (!MediaPlayer)
	{
		return false;
	}

	ULevelSequence* LevelSequence = GetLevelSequence();

	if (!LevelSequence)
	{
		return false;
	}

	UMovieScene* MovieScene = LevelSequence->GetMovieScene();

	if (!MovieScene)
	{
		return false;
	}

	UWorld* World = LevelSequence->GetWorld();

	if (!World)
	{
		return false;
	}

	if (InMediaStream->GetWorld() != World)
	{
		return false;
	}

	TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState = MovieSceneHelpers::CreateTransientSharedPlaybackState(World, LevelSequence);
	const FGuid MediaStreamId = LevelSequence->FindBindingFromObject(InMediaStream, SharedPlaybackState);

	if (MediaStreamId.IsValid())
	{
		return !MovieScene->FindTrack<UMovieSceneMediaTrack>(MediaStreamId);
	}

	return LevelSequence->CanPossessObject(*InMediaStream, World);
}

bool FMediaStreamEditorSequencerLibrary::AddTrack(UMediaStream* InMediaStream)
{
	if (!InMediaStream || !CanAddTrack(InMediaStream))
	{
		return false;
	}

	IMediaStreamPlayer* MediaStreamPlayerIntf = InMediaStream->GetPlayer().GetInterface();

	if (!MediaStreamPlayerIntf || MediaStreamPlayerIntf->IsReadOnly())
	{
		return false;
	}

	UMediaPlayer* MediaPlayer = MediaStreamPlayerIntf->GetPlayer();

	if (!MediaPlayer)
	{
		return false;
	}

	ULevelSequence* LevelSequence = GetLevelSequence();

	if (!LevelSequence)
	{
		return false;
	}

	UMovieScene* MovieScene = LevelSequence->GetMovieScene();

	if (!MovieScene)
	{
		return false;
	}

	UWorld* World = LevelSequence->GetWorld();

	if (!World)
	{
		return false;
	}

	if (InMediaStream->GetWorld() != World)
	{
		return false;
	}

	TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState = MovieSceneHelpers::CreateTransientSharedPlaybackState(World, LevelSequence);
	FGuid MediaStreamId = LevelSequence->FindBindingFromObject(InMediaStream, SharedPlaybackState);

	const FString MediaPlayerPath = MediaPlayer->GetPathName();

	if (!MediaStreamId.IsValid())
	{
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		IAssetEditorInstance* AssetEditor = AssetEditorSubsystem->FindEditorForAsset(LevelSequence, /* Focus */ false);
		ILevelSequenceEditorToolkit* LevelSequenceEditor = static_cast<ILevelSequenceEditorToolkit*>(AssetEditor);
		TSharedPtr<ISequencer> Sequencer = LevelSequenceEditor ? LevelSequenceEditor->GetSequencer() : nullptr;

		if (!Sequencer.IsValid())
		{
			return false;
		}

		MediaStreamId = FSequencerUtilities::CreateBinding(Sequencer.ToSharedRef(), *InMediaStream);

		if (!MediaStreamId.IsValid())
		{
			return false;
		}
	}

 	if (MovieScene->FindTrack<UMovieSceneMediaTrack>(MediaStreamId))
	{
		return false;
	}

	UMovieSceneMediaTrack* MediaTrack = MovieScene->AddTrack<UMovieSceneMediaTrack>(MediaStreamId);

	if (!MediaTrack)
	{
		return false;
	}

	TRange<FFrameNumber> Range = MovieScene->GetPlaybackRange();

	if (Range.GetLowerBound().IsOpen())
	{
		Range.SetLowerBound(FFrameNumber(0));
	}

	if (Range.GetUpperBound().IsOpen() || Range.GetUpperBound().GetValue() == 0)
	{
		Range.SetUpperBound(FFrameNumber(250));
	}

	UMovieSceneMediaSection* MediaSection = Cast<UMovieSceneMediaSection>(MediaTrack->CreateNewSection());
	MediaSection->bUseExternalMediaPlayer = true;
	MediaSection->ExternalMediaPlayer = MediaPlayer;
	MediaSection->SetStartFrame(Range.GetLowerBound());
	MediaSection->SetEndFrame(Range.GetUpperBound());
	MediaSection->SetMediaSourceProxy(UE::MovieScene::FRelativeObjectBindingID(MediaStreamId), /* Proxy Index */ 0);

	MediaTrack->SetDisplayName(LOCTEXT("MediaTrack", "Media Track"));

	MediaTrack->AddSection(*MediaSection);

	return true;
}

#undef LOCTEXT_NAMESPACE
