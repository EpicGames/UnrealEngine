// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class UObject;
class ULevelSequence;
class UMediaStream;

class FMediaStreamEditorSequencerLibrary
{
public:
	static ULevelSequence* GetLevelSequence();
	
	static bool HasTrack(UMediaStream* InMediaStream);

	static bool CanAddTrack(UMediaStream* InMediaStream);

	static bool AddTrack(UMediaStream* InMediaStream);
};
