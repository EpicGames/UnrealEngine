// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaStreamEditorStyle.h"

#include "Brushes/SlateBorderBrush.h"
#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Fonts/SlateFontInfo.h"
#include "Interfaces/IPluginManager.h"
#include "Math/Vector2D.h"
#include "Misc/Paths.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"

/* FMediaStreamEditorStyle structors
 *****************************************************************************/

FMediaStreamEditorStyle::FMediaStreamEditorStyle()
	: FSlateStyleSet("MediaStreamEditorStyle")
{
	const FVector2D Icon12x12(12.0f, 12.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME))
	{
		SetContentRoot(FPaths::Combine(Plugin->GetBaseDir(), TEXT("Resources")));

		// Class icons.
		Set("ClassIcon.MediaStream", new IMAGE_BRUSH_SVG(TEXT("Icons/MediaStream"), Icon16x16));
		Set("ClassThumbnail.MediaStream", new IMAGE_BRUSH_SVG(TEXT("Icons/MediaStream"), Icon64x64));
		Set("ClassIcon.MediaStreamActor", new IMAGE_BRUSH_SVG(TEXT("Icons/MediaStream"), Icon16x16));
		Set("ClassThumbnail.MediaStreamActor", new IMAGE_BRUSH_SVG(TEXT("Icons/MediaStream"), Icon64x64));
		Set("ClassIcon.MediaStreamComponent", new IMAGE_BRUSH_SVG(TEXT("Icons/MediaStream"), Icon16x16));
		Set("ClassThumbnail.MediaStreamComponent", new IMAGE_BRUSH_SVG(TEXT("Icons/MediaStream"), Icon64x64));
	}

	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Media/MediaPlayerEditor/Content"));

	// toolbar icons
	Set("MediaStreamEditor.CloseMedia", new IMAGE_BRUSH("icon_eject_40x", Icon40x40));
	Set("MediaStreamEditor.CloseMedia.Small", new IMAGE_BRUSH("icon_eject_40x", Icon20x20));
	Set("MediaStreamEditor.ForwardMedia", new IMAGE_BRUSH("icon_rewind_40x", Icon40x40));
	Set("MediaStreamEditor.ForwardMedia.Small", new IMAGE_BRUSH("icon_rewind_40x", Icon20x20));
	Set("MediaStreamEditor.NextMedia", new IMAGE_BRUSH("icon_step_40x", Icon40x40));
	Set("MediaStreamEditor.NextMedia.Small", new IMAGE_BRUSH("icon_step_40x", Icon20x20));
	Set("MediaStreamEditor.OpenMedia", new IMAGE_BRUSH("icon_open_40x", Icon40x40));
	Set("MediaStreamEditor.OpenMedia.Small", new IMAGE_BRUSH("icon_open_40x", Icon20x20));
	Set("MediaStreamEditor.PauseMedia", new IMAGE_BRUSH("icon_pause_40x", Icon40x40));
	Set("MediaStreamEditor.PauseMedia.Small", new IMAGE_BRUSH("icon_pause_40x", Icon20x20));
	Set("MediaStreamEditor.PlayMedia", new IMAGE_BRUSH("icon_play_40x", Icon40x40));
	Set("MediaStreamEditor.PlayMedia.Small", new IMAGE_BRUSH("icon_play_40x", Icon20x20));
	Set("MediaStreamEditor.PreviousMedia", new IMAGE_BRUSH("icon_step_back_40x", Icon40x40));
	Set("MediaStreamEditor.PreviousMedia.Small", new IMAGE_BRUSH("icon_step_back_40x", Icon20x20));
	Set("MediaStreamEditor.ReverseMedia", new IMAGE_BRUSH("icon_reverse_40x", Icon40x40));
	Set("MediaStreamEditor.ReverseMedia.Small", new IMAGE_BRUSH("icon_reverse_40x", Icon20x20));
	Set("MediaStreamEditor.RewindMedia", new IMAGE_BRUSH("icon_rewind_40x", Icon40x40));
	Set("MediaStreamEditor.RewindMedia.Small", new IMAGE_BRUSH("icon_rewind_40x", Icon20x20));
	Set("MediaStreamEditor.StopMedia", new IMAGE_BRUSH("icon_stop_40x", Icon40x40));
	Set("MediaStreamEditor.StopMedia.Small", new IMAGE_BRUSH("icon_stop_40x", Icon20x20));

	// scrubber
	Set("MediaPlayerEditor.Scrubber", FSliderStyle()
		.SetNormalBarImage(FSlateColorBrush(FColor::White))
		.SetDisabledBarImage(FSlateColorBrush(FLinearColor::Gray))
		.SetNormalThumbImage(IMAGE_BRUSH("scrubber", FVector2D(2.0f, 10.0f)))
		.SetHoveredThumbImage(IMAGE_BRUSH("scrubber", FVector2D(2.0f, 10.0f)))
		.SetDisabledThumbImage(IMAGE_BRUSH("scrubber", FVector2D(2.0f, 10.0f)))
		.SetBarThickness(2.0f)
	);

	Set("MediaButtons", FButtonStyle(FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Animation.PlayControlsButton"))
		.SetNormal(FSlateNoResource())
		.SetDisabled(FSlateNoResource())
		.SetHovered(FSlateRoundedBoxBrush(FLinearColor(0.2f, 0.2f, 0.2f, 0.5f), 3.f, FVector2D(20.f)))
		.SetPressed(FSlateRoundedBoxBrush(FLinearColor(0.1f, 0.1f, 0.1f, 0.5f), 3.f, FVector2D(20.f)))
		.SetNormalPadding(FMargin(2.f, 2.f, 2.f, 2.f))
		.SetPressedPadding(FMargin(2.f, 2.f, 2.f, 2.f)));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FMediaStreamEditorStyle::~FMediaStreamEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
