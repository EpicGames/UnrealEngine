// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

/**
 * Implements the visual style of the media plate editor UI.
 */
class FMediaStreamEditorStyle : public FSlateStyleSet
{
public:
	/** Default constructor. */
	FMediaStreamEditorStyle();

	 /** Destructor. */
	~FMediaStreamEditorStyle();

	static TSharedRef<FMediaStreamEditorStyle> Get()
	{
		static TSharedPtr<FMediaStreamEditorStyle> Singleton;

		if (!Singleton.IsValid())
		{
			Singleton = MakeShareable(new FMediaStreamEditorStyle);
		}

		return Singleton.ToSharedRef();
	}
};
