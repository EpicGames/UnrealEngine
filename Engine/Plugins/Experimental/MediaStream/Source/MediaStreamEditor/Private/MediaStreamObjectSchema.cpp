// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaStreamObjectSchema.h"

#include "GameFramework/Actor.h"
#include "MediaStream.h"

#define LOCTEXT_NAMESPACE "MediaStreamObjectSchema"

UObject* FMediaStreamObjectSchema::GetParentObject(UObject* InObject) const
{
	/*
	 * Need to re-enable this, but fix the binding bugs first.
	 * Using this schema changes the binding path to be based on the actor,
	 * but the context of the binding remainds the world, so it fails to
	 * resolve.
	 *
	if (InObject)
	{
		if (AActor* Actor = InObject->GetTypedOuter<AActor>())
		{
			return Actor;
		}
	}
	 */

	return nullptr;
}

UE::Sequencer::FObjectSchemaRelevancy FMediaStreamObjectSchema::GetRelevancy(const UObject* InObject) const
{
	if (InObject && InObject->IsA<UMediaStream>())
	{
		return UMediaStream::StaticClass();
	}

	return UE::Sequencer::FObjectSchemaRelevancy();
}

TSharedPtr<FExtender> FMediaStreamObjectSchema::ExtendObjectBindingMenu(TSharedRef<FUICommandList> InCommandList
	, TWeakPtr<ISequencer> InSequencerWeak
	, TConstArrayView<UObject*> InContextSensitiveObjects) const
{
	return nullptr;
}

FText FMediaStreamObjectSchema::GetPrettyName(const UObject* InObject) const
{
	return LOCTEXT("Media", "Media");
}

#undef LOCTEXT_NAMESPACE
