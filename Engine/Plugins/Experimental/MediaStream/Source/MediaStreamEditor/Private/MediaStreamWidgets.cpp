// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaStreamWidgets.h"

#include "DetailsPanel/MediaStreamSourceCustomization.h"
#include "DetailsPanel/SMediaStreamMediaDetails.h"
#include "DetailsPanel/SMediaStreamMediaTrack.h"
#include "DetailsPanel/SMediaStreamPlaybackControls.h"
#include "Templates/SharedPointer.h"

namespace UE::MediaStreamEditor
{

IMediaStreamSchemeHandler::FCustomWidgets FMediaStreamWidgets::GenerateSourceSchemeRows(UMediaStream* InMediaStream)
{
	return FMediaStreamSourceCustomization().Customize(InMediaStream);
}

TSharedRef<SWidget> FMediaStreamWidgets::CreateTextureDetailsWidget(UMediaStream* InMediaStream)
{
	return SNew(SMediaStreamMediaDetails, InMediaStream);
}

TSharedRef<SWidget> FMediaStreamWidgets::CreateTrackWidget(const TArray<UMediaStream*>& InMediaStreams)
{
	return SNew(SMediaStreamMediaTrack, InMediaStreams);
}

TSharedRef<SWidget> FMediaStreamWidgets::CreateControlsWidget(const TArray<UMediaStream*>& InMediaStreams)
{
	return SNew(SMediaStreamPlaybackControls, InMediaStreams);
}

}; // UE::MediaStreamEditor
