// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaStreamSchemeHandler.h"
#include "Templates/SharedPointerFwd.h"

class SWidget;
class UMediaStream;

namespace UE::MediaStreamEditor
{

class FMediaStreamWidgets
{
public:
	/**
	 * Generates custom widgets for source scheme types.
	 */
	MEDIASTREAMEDITOR_API static IMediaStreamSchemeHandler::FCustomWidgets GenerateSourceSchemeRows(UMediaStream* InMediaStream);

	/**
	 * Creates a widget representing the details of the owned Media Texture.
	 */
	MEDIASTREAMEDITOR_API static TSharedRef<SWidget> CreateTextureDetailsWidget(UMediaStream* InMediaStreams);

	/**
	 * Creates a widget allowing the user to scrub the media.
	 */
	MEDIASTREAMEDITOR_API static TSharedRef<SWidget> CreateTrackWidget(const TArray<UMediaStream*>& InMediaStreams);

	/**
	 * Creates a widget allowing the user to control the media playback.
	 * If providing multiple media streams, the first should be the one that can have tracks.
	 */
	MEDIASTREAMEDITOR_API static TSharedRef<SWidget> CreateControlsWidget(const TArray<UMediaStream*>& InMediaStreams);
};

} // UE::MediaStreamEditor
